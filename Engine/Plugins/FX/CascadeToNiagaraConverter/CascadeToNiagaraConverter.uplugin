{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Cascade To Niagara Converter",
	"Description": "Add support for scriptable conversion of Cascade Systems to Niagara Systems.",
	"Category": "FX",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
		  "Name": "CascadeToNiagaraConverter",
		  "Type": "Editor",
		  "LoadingPhase": "Default"
		}
	],

	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		}
	]
}
