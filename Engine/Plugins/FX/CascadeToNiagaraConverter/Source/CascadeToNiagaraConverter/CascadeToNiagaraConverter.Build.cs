// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CascadeToNiagaraConverter : ModuleRules
{
	public CascadeToNiagaraConverter(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"MessageLog",
			}
		);	
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
              	"NiagaraCore",
				"Engine",
				"AssetRegistry",
				"Niagara",
				"NiagaraEditor",
				"CoreUObject",
				"Slate",
				"SlateCore",
				"PythonScriptPlugin",
			}
		);

        PublicIncludePathModuleNames.AddRange(
            new string[]
            {
				"Sequencer",
			}
		);

	}
}
