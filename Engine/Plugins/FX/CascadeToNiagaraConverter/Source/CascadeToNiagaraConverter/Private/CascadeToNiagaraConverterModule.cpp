// Copyright Epic Games, Inc. All Rights Reserved.

#include "CascadeToNiagaraConverterModule.h"
#include "ContentBrowserModule.h"
#include "IPythonScriptPlugin.h"
#include "NiagaraAnalytics.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NiagaraMessageManager.h"
#include "NiagaraStackGraphUtilitiesAdapterLibrary.h"
#include "PythonScriptTypes.h"

IMPLEMENT_MODULE(ICascadeToNiagaraConverterModule, CascadeToNiagaraConverter);
DEFINE_LOG_CATEGORY(LogFXConverter);

#define LOCTEXT_NAMESPACE "CascadeToNiagaraConverterModule"


const FName FNiagaraConverterMessageTopics::VerboseConversionEventTopicName = "VerboseConversionEvent";
const FName FNiagaraConverterMessageTopics::ConversionEventTopicName = "ConversionEvent";

void ICascadeToNiagaraConverterModule::StartupModule()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.GetAllAssetViewContextMenuExtenders().Add(FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&ICascadeToNiagaraConverterModule::OnExtendContentBrowserAssetSelectionMenu));
	FNiagaraMessageManager* MessageManager = FNiagaraMessageManager::Get();
	MessageManager->RegisterMessageTopic(FNiagaraConverterMessageTopics::VerboseConversionEventTopicName);
	MessageManager->RegisterMessageTopic(FNiagaraConverterMessageTopics::ConversionEventTopicName);
	MessageManager->RegisterAdditionalMessageLogTopic(FNiagaraConverterMessageTopics::ConversionEventTopicName);
}

void ICascadeToNiagaraConverterModule::ShutdownModule()
{
}

TSharedRef<FExtender> ICascadeToNiagaraConverterModule::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	Extender->AddMenuExtension(
		"GetAssetActions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateStatic(&AddMenuExtenderConvertEntry, SelectedAssets)
		);

	return Extender;
}

void ICascadeToNiagaraConverterModule::AddMenuExtenderConvertEntry(FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets)
{
	if (SelectedAssets.Num() > 0)
	{
		// check that all selected assets are cascade systems.
		TArray<UParticleSystem*> CascadeSystems;
		for (const FAssetData& SelectedAsset : SelectedAssets)
		{
			if (!SelectedAsset.IsInstanceOf(UParticleSystem::StaticClass()))
			{
				return;
			}
			CascadeSystems.Add(static_cast<UParticleSystem*>(SelectedAsset.GetAsset()));
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ConvertToNiagaraSystem", "Convert To Niagara System"),
			LOCTEXT("ConvertToNiagaraSystem_Tooltip", "Duplicate and convert the selected Cascade Systems to equivalent Niagara Systems."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&ExecuteConvertCascadeSystemToNiagaraSystem, CascadeSystems))
		);
	}
}

void ICascadeToNiagaraConverterModule::ExecuteConvertCascadeSystemToNiagaraSystem(TArray<UParticleSystem*> CascadeSystems)
{
	UConvertCascadeToNiagaraResults* Results = NewObject<UConvertCascadeToNiagaraResults>();

	bool bCancelled = false;
	bool bSuccess = true;
	double StartTime = FPlatformTime::Seconds();
	for (UParticleSystem* CascadeSystem : CascadeSystems)
	{
		Results->Init();
		FPythonCommandEx PythonCommand = FPythonCommandEx();
		PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteFile;

		PythonCommand.Command = "../../Plugins/FX/CascadeToNiagaraConverter/Content/Python/ConvertCascadeToNiagara.py ";
		PythonCommand.Command.Append(*CascadeSystem->GetPathName());
		PythonCommand.Command.Append(" ");
		PythonCommand.Command.Append(*Results->GetPathName());
		IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand);

		if (Results->bCancelledByUser)
		{
			bCancelled = true;
			break;
		}
		if (Results->bCancelledByPythonError)
		{
			// raise modal dialog of status
			//PythonCommand.CommandResult
			bSuccess = false;
			break;
		}
	}

	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Emplace(TEXT("DurationSeconds"), FPlatformTime::Seconds() - StartTime);
	Attributes.Emplace(TEXT("Cancelled"), bCancelled);
	Attributes.Emplace(TEXT("Error"), bSuccess == false);
	Attributes.Emplace(TEXT("AssetCount"), CascadeSystems.Num());
	NiagaraAnalytics::RecordEvent("Cascade.Conversion", Attributes);
}

#undef LOCTEXT_NAMESPACE //"CascadeToNiagaraConverterModule"
