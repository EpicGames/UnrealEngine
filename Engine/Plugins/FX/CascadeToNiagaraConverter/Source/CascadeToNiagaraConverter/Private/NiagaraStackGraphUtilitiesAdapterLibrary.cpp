// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackGraphUtilitiesAdapterLibrary.h"
#include "Distributions/DistributionFloatParameterBase.h"
#include "NiagaraSystem.h"
#include "Distributions/DistributionVectorParameterBase.h"
#include "NiagaraEditorUtilities.h"
#include "Engine/SkeletalMesh.h"
#include "Particles/Acceleration/ParticleModuleAcceleration.h"
#include "Engine/StaticMesh.h"
#include "Particles/Acceleration/ParticleModuleAccelerationDrag.h"
#include "NiagaraGraph.h"
#include "Particles/Attractor/ParticleModuleAttractorLine.h"
#include "NiagaraNodeInput.h"
#include "Particles/Attractor/ParticleModuleAttractorParticle.h"
#include "Particles/Acceleration/ParticleModuleAccelerationDragScaleOverLife.h"
#include "Particles/Attractor/ParticleModuleAttractorPoint.h"
#include "Particles/Acceleration/ParticleModuleAccelerationOverLifetime.h"
#include "Particles/Beam/ParticleModuleBeamSource.h"
#include "Particles/Beam/ParticleModuleBeamTarget.h"
#include "Particles/Collision/ParticleModuleCollision.h"
#include "Particles/Camera/ParticleModuleCameraOffset.h"
#include "Particles/Color/ParticleModuleColor.h"
#include "Particles/Color/ParticleModuleColorOverLife.h"
#include "Particles/Color/ParticleModuleColorScaleOverLife.h"
#include "Particles/Kill/ParticleModuleKillBox.h"
#include "Particles/Lifetime/ParticleModuleLifetime.h"
#include "Particles/Light/ParticleModuleLight.h"
#include "Particles/Location/ParticleModuleLocation.h"
#include "Particles/Location/ParticleModuleLocationDirect.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveCylinder.h"
#include "Particles/Location/ParticleModuleLocationPrimitiveSphere.h"
#include "Particles/Location/ParticleModuleLocationWorldOffset.h"
#include "Particles/Location/ParticleModuleLocationSkelVertSurface.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Particles/Modules/Location/ParticleModulePivotOffset.h"
#include "Particles/Orientation/ParticleModuleOrientationAxisLock.h"
#include "Particles/Rotation/ParticleModuleRotation.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/Rotation/ParticleModuleRotationOverLifetime.h"
#include "Particles/Rotation/ParticleModuleMeshRotation.h"
#include "Particles/RotationRate/ParticleModuleMeshRotationRate.h"
#include "Particles/RotationRate/ParticleModuleMeshRotationRateMultiplyLife.h"
#include "Particles/RotationRate/ParticleModuleRotationRate.h"
#include "Particles/RotationRate/ParticleModuleMeshRotationRateOverLife.h"
#include "Particles/RotationRate/ParticleModuleRotationRateMultiplyLife.h"
#include "Particles/Size/ParticleModuleSize.h"
#include "Particles/Size/ParticleModuleSizeScale.h"
#include "Particles/Size/ParticleModuleSizeScaleBySpeed.h"
#include "Particles/Size/ParticleModuleSizeMultiplyLife.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/Spawn/ParticleModuleSpawnPerUnit.h"
#include "Particles/SubUV/ParticleModuleSubUVMovie.h"
#include "Particles/Trail/ParticleModuleTrailSource.h"
#include "Particles/VectorField/ParticleModuleVectorFieldLocal.h"
#include "Particles/VectorField/ParticleModuleVectorFieldRotationRate.h"
#include "Particles/Velocity/ParticleModuleVelocity.h"
#include "Particles/Velocity/ParticleModuleVelocityCone.h"
#include "Particles/Velocity/ParticleModuleVelocityInheritParent.h"
#include "Particles/Velocity/ParticleModuleVelocityOverLifetime.h"
#include "Particles/Acceleration/ParticleModuleAccelerationConstant.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/TypeData/ParticleModuleTypeDataRibbon.h"

#include "Particles/ParticleLODLevel.h"
#include "NiagaraScriptSource.h"
#include "NiagaraClipboard.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"

#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackEventScriptItemGroup.h"
#include "NiagaraRibbonRendererProperties.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraLightRendererProperties.h"
#include "NiagaraComponentRendererProperties.h"
#include "NiagaraDataInterfaceCurve.h"
#include "NiagaraDataInterfaceVector2DCurve.h"
#include "NiagaraDataInterfaceVectorCurve.h"
#include "NiagaraDataInterfaceVector4Curve.h"
#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "NiagaraMessages.h"
#include "NiagaraSettings.h"

#include "Distributions/DistributionFloatConstantCurve.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionFloatUniformCurve.h"

#include "Distributions/DistributionVectorConstantCurve.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Distributions/DistributionVectorUniformCurve.h"
#include "NiagaraNodeFunctionCall.h"
#include "CascadeToNiagaraConverterModule.h"
#include "Engine/UserDefinedEnum.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackGraphUtilitiesAdapterLibrary)

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	UFXConverterUtilitiesLibrary																			  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FName UFXConverterUtilitiesLibrary::GetNiagaraScriptInputTypeName(ENiagaraScriptInputType InputType)
{
	switch (InputType) {
	case ENiagaraScriptInputType::Int:
		return FNiagaraTypeDefinition::GetIntDef().GetFName();
	case ENiagaraScriptInputType::Float:
		return FNiagaraTypeDefinition::GetFloatDef().GetFName();
	case ENiagaraScriptInputType::Vec2:
		return FNiagaraTypeDefinition::GetVec2Def().GetFName();
	case ENiagaraScriptInputType::Vec3:
		return FNiagaraTypeDefinition::GetVec3Def().GetFName();
	case ENiagaraScriptInputType::Vec4:
		return FNiagaraTypeDefinition::GetVec4Def().GetFName();
	case ENiagaraScriptInputType::LinearColor:
		return FNiagaraTypeDefinition::GetColorDef().GetFName();
	case ENiagaraScriptInputType::Matrix:
		return FNiagaraTypeDefinition::GetMatrix4Def().GetFName();
	case ENiagaraScriptInputType::Quaternion:
		return FNiagaraTypeDefinition::GetQuatDef().GetFName();
	case ENiagaraScriptInputType::Bool:
		return FNiagaraTypeDefinition::GetBoolDef().GetFName();
	case ENiagaraScriptInputType::Position:
		return FNiagaraTypeDefinition::GetPositionDef().GetFName();
	};
	ensureMsgf(false, TEXT("Tried to get FName for unknown ENiagaraScriptInputType!"));
	return FName();
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationPrimitiveBaseProps(
	UParticleModuleLocationPrimitiveBase* ParticleModule
	, bool& bOutPositiveX
	, bool& bOutPositiveY
	, bool& bOutPositiveZ
	, bool& bOutNegativeX
	, bool& bOutNegativeY
	, bool& bOutNegativeZ
	, bool& bOutSurfaceOnly
	, bool& bOutVelocity
	, UDistribution*& OutVelocityScale
	, UDistribution*& OutStartLocation)
{
	bOutPositiveX = ParticleModule->Positive_X;
	bOutPositiveY = ParticleModule->Positive_Y;
	bOutPositiveZ = ParticleModule->Positive_Z;
	bOutNegativeX = ParticleModule->Negative_X;
	bOutNegativeY = ParticleModule->Negative_Y;
	bOutNegativeZ  = ParticleModule->Negative_Z;
	bOutSurfaceOnly = ParticleModule->SurfaceOnly;
	bOutVelocity = ParticleModule->Velocity;
	OutVelocityScale = ParticleModule->VelocityScale.Distribution;
	OutStartLocation = ParticleModule->StartLocation.Distribution;
}

TArray<UParticleEmitter*> UFXConverterUtilitiesLibrary::GetCascadeSystemEmitters(const UParticleSystem* System)
{
	return System->Emitters;
}

float UFXConverterUtilitiesLibrary::GetWarmupTime(const UParticleSystem* System)
{
	return System->WarmupTime;
}

UParticleLODLevel* UFXConverterUtilitiesLibrary::GetCascadeEmitterLodLevel(UParticleEmitter* Emitter, const int32 Idx)
{
	return Emitter->GetLODLevel(Idx);
}

bool UFXConverterUtilitiesLibrary::GetLodLevelIsEnabled(UParticleLODLevel* LodLevel)
{
	return LodLevel->bEnabled;
}

TArray<UParticleModule*> UFXConverterUtilitiesLibrary::GetLodLevelModules(UParticleLODLevel* LodLevel)
{
	return LodLevel->Modules;
}

UParticleModuleSpawn* UFXConverterUtilitiesLibrary::GetLodLevelSpawnModule(UParticleLODLevel* LodLevel)
{
	return LodLevel->SpawnModule;
}

UParticleModuleRequired* UFXConverterUtilitiesLibrary::GetLodLevelRequiredModule(UParticleLODLevel* LodLevel)
	{
	return LodLevel->RequiredModule;
	}

UParticleModuleTypeDataBase* UFXConverterUtilitiesLibrary::GetLodLevelTypeDataModule(UParticleLODLevel* LodLevel)
{
	return LodLevel->TypeDataModule;
}

bool UFXConverterUtilitiesLibrary::GetParticleModuleIsEnabled(UParticleModule* Module)
{
	return Module->bEnabled;
}

FName UFXConverterUtilitiesLibrary::GetCascadeEmitterName(UParticleEmitter* Emitter)
{
	return Emitter->GetEmitterName();
}

int32 UFXConverterUtilitiesLibrary::GetCascadeEmitterRenderMode(UParticleEmitter* Emitter)
{
	return Emitter->EmitterRenderMode;
}

FAssetData UFXConverterUtilitiesLibrary::CreateAssetData(FString InPath)
{
	FAssetData Out;
	FSoftObjectPath Path;
	Path.SetPath(InPath);
	UObject* Obj = Path.TryLoad();
	if (Obj)
	{
		Out = FAssetData(Obj);
	}
	else
	{
		UE_LOGF(LogScript, Error, "Failed to make path %ls" , *InPath);
	}
	
	return Out;
}

UNiagaraScriptConversionContext* UFXConverterUtilitiesLibrary::CreateScriptContext(const FCreateScriptContextArgs& Args)
{
	UNiagaraScriptConversionContext* ScriptContext = NewObject<UNiagaraScriptConversionContext>();
	if (Args.bScriptVersionSet || Args.ScriptVersion.MajorVersion != 0 || Args.ScriptVersion.MinorVersion != 0)
	{
		ScriptContext->Init(Args.ScriptAsset, Args.ScriptVersion);
	}
	else
	{
		ScriptContext->Init(Args.ScriptAsset);
	}
	
	return ScriptContext;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputLinkedEnum(const FString& ParameterNameString, const FString& UserDefinedEnumAssetPath)
{
	UUserDefinedEnum* UserDefinedEnum = LoadObject<UUserDefinedEnum>(nullptr, *UserDefinedEnumAssetPath);
	if (UserDefinedEnum == nullptr)
	{
		return nullptr;
	}

	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateEnumLinkedValueInput(GetTransientPackage(), FName(), false, false, UserDefinedEnum, FName(ParameterNameString));
	if (NewInput != nullptr)
	{
		UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
		Input->Init(NewInput, ENiagaraScriptInputType::Enum, NewInput->GetTypeDef());
		return Input;
	}
	return nullptr;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputLinkedParameter(FString ParameterNameString, ENiagaraScriptInputType InputType)
{
	ensureMsgf(InputType != ENiagaraScriptInputType::Enum, TEXT("Use CreateEnumLinkedValueInput for Enum linked parameters"));

	const FName InputTypeName = GetNiagaraScriptInputTypeName(InputType);
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateLinkedValueInput(GetTransientPackage(), FName(), InputTypeName, false, false, FName(ParameterNameString));
	const FNiagaraTypeDefinition& TargetTypeDef = UNiagaraClipboardEditorScriptingUtilities::GetRegisteredTypeDefinitionByName(InputTypeName);
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, InputType, TargetTypeDef); 
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputLinkedParameterEx(FString ParameterNameString, UNiagaraScriptConversionContextInput* InBaseInput)
{
	const FName InputTypeName = InBaseInput->TypeDefinition.GetFName();
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateLinkedValueInput(GetTransientPackage(), FName(), InputTypeName, false, false, FName(ParameterNameString));
//	const FNiagaraTypeDefinition& TargetTypeDef = UNiagaraClipboardEditorScriptingUtilities::GetRegisteredTypeDefinitionByName(InputTypeName);
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Enum, InBaseInput->TypeDefinition); 
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputFloat(float Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateFloatLocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetFloatDef();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Float, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputLinearColor(FLinearColor Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateColorLocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetColorDef();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::LinearColor, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputMatrix(FMatrix44f Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateMatrixLocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetMatrix4Def();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Matrix, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputQuat(FQuat4f Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateQuatLocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetQuatDef();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Quaternion, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputVec2(FVector2D Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateVec2LocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetVec2Def();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Vec2, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputVec4(FVector4f Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateVec4LocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetVec4Def();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Vec4, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputVector(FVector Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateVec3LocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetVec3Def();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Vec3, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputStruct(UUserDefinedStruct* Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateStructLocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	if (NewInput != nullptr)
	{
		UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
		Input->Init(NewInput, ENiagaraScriptInputType::Struct, NewInput->GetTypeDef());
		return Input;
	}
	return nullptr;
}

static UNiagaraClipboardFunctionInput* CreateEnumLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditCoditionValue, UEnum* InEnumType, int32 InEnumValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition(InEnumType);
	TArray<uint8> EnumValue;
	EnumValue.AddUninitialized(sizeof(int32));
	FMemory::Memcpy(EnumValue.GetData(), &InEnumValue, sizeof(int32));

	return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateLocalValue(
		InOuter != nullptr ? InOuter : GetTransientPackage(),
		InInputName,
		InputType,
		bInHasEditCondition ? TOptional<bool>(bInEditCoditionValue) : TOptional<bool>(),
		EnumValue)
	);
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputEnum(const FString& UserDefinedEnumAssetPath, const FString& UserDefinedEnumValueNameString)
{
	UUserDefinedEnum* UserDefinedEnum = LoadObject<UUserDefinedEnum>(nullptr, *UserDefinedEnumAssetPath, nullptr);
	if (UserDefinedEnum == nullptr)
	{
		return nullptr;
	}
	const int64 UserDefinedEnumValue = UserDefinedEnum->GetValueByNameString(UserDefinedEnumValueNameString, EGetByNameFlags::CheckAuthoredName);
	if (UserDefinedEnumValue == INDEX_NONE)
	{
		return nullptr;
	}
	else if (UserDefinedEnumValue > INT32_MAX)
	{
		ensureMsgf(false, TEXT("Guarding against int64 enums, Niagara may not interact properly with this."));
		return nullptr;
	}

	const int32 Int32UserDefinedEnumValue = UserDefinedEnumValue;
	UEnum* Enum = UserDefinedEnum;
	if (UserDefinedEnumAssetPath == TEXT("/Niagara/Enums/ENiagaraCoordinateSpace.ENiagaraCoordinateSpace"))
	{
		Enum = StaticEnum<ENiagaraCoordinateSpace>();
	}
	UNiagaraClipboardFunctionInput* NewInput = CreateEnumLocalValueInput(GetTransientPackage(), FName(), false, false, Enum, Int32UserDefinedEnumValue);
	if (NewInput != nullptr)
	{
		UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
		Input->Init(NewInput, ENiagaraScriptInputType::Enum, NewInput->GetTypeDef());
		return Input;
	}
	return nullptr;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputInt(int32 Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateIntLocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetIntDef();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Int, TargetTypeDef);
	return Input;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputDynamic(UNiagaraScriptConversionContext* DynamicInputScriptContext, ENiagaraScriptInputType InputType)
{
	const FName InputTypeName = GetNiagaraScriptInputTypeName(InputType);
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateDynamicValueInput(
		GetTransientPackage()
		, FName()
		, InputTypeName
		, false
		, false
		, FString()
		, DynamicInputScriptContext->GetScript());
	
	// copy over the original function inputs to the new dynamic input script associated with this clipboard function input
	if (NewInput)
	{
		NewInput->Dynamic->Inputs = DynamicInputScriptContext->GetClipboardFunctionInputs();
		const FNiagaraTypeDefinition& TargetTypeDef = UNiagaraClipboardEditorScriptingUtilities::GetRegisteredTypeDefinitionByName(InputTypeName);
		UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
		Input->Init(NewInput, InputType, TargetTypeDef);
		Input->StackMessages = DynamicInputScriptContext->GetStackMessages();
		return Input;
	}
	else 
	{
		return nullptr;
	}
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputDI(UNiagaraDataInterface* Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateDataValueInput(
		GetTransientPackage()
		, FName()
		, false
		, false
		, Value);

	if (NewInput != nullptr)
	{
		UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
		Input->Init(NewInput, ENiagaraScriptInputType::DataInterface, NewInput->GetTypeDef());
		return Input;
	}
	return nullptr;
}

UNiagaraScriptConversionContextInput* UFXConverterUtilitiesLibrary::CreateScriptInputBool(bool Value)
{
	UNiagaraClipboardFunctionInput* NewInput = UNiagaraClipboardEditorScriptingUtilities::CreateBoolLocalValueInput(GetTransientPackage(), FName(), false, false, Value);
	const FNiagaraTypeDefinition& TargetTypeDef = FNiagaraTypeDefinition::GetBoolDef();
	UNiagaraScriptConversionContextInput* Input = NewObject<UNiagaraScriptConversionContextInput>();
	Input->Init(NewInput, ENiagaraScriptInputType::Bool, TargetTypeDef);
	return Input;
}

UNiagaraRibbonRendererProperties* UFXConverterUtilitiesLibrary::CreateRibbonRendererProperties()
{
	return NewObject<UNiagaraRibbonRendererProperties>();
}

UNiagaraMeshRendererProperties* UFXConverterUtilitiesLibrary::CreateMeshRendererProperties()
{
	return NewObject<UNiagaraMeshRendererProperties>();
}

UNiagaraLightRendererProperties* UFXConverterUtilitiesLibrary::CreateLightRendererProperties()
{
	return NewObject<UNiagaraLightRendererProperties>();
}

UNiagaraComponentRendererProperties* UFXConverterUtilitiesLibrary::CreateComponentRendererProperties()
{
	return NewObject<UNiagaraComponentRendererProperties>();
}

UNiagaraDataInterfaceSkeletalMesh* UFXConverterUtilitiesLibrary::CreateSkeletalMeshDataInterface()
{
	return NewObject<UNiagaraDataInterfaceSkeletalMesh>();
}

UNiagaraDataInterfaceCurve* UFXConverterUtilitiesLibrary::CreateFloatCurveDI(TArray<FRichCurveKeyBP> Keys)
{
	UNiagaraDataInterfaceCurve* DI_Curve = NewObject<UNiagaraDataInterfaceCurve>();
	const TArray<FRichCurveKey> BaseKeys = FRichCurveKeyBP::KeysToBase(Keys);
	DI_Curve->Curve.SetKeys(BaseKeys);
	return DI_Curve;
}

UNiagaraDataInterfaceVector2DCurve* UFXConverterUtilitiesLibrary::CreateVec2CurveDI(TArray<FRichCurveKeyBP> X_Keys, TArray<FRichCurveKeyBP> Y_Keys)
{
	UNiagaraDataInterfaceVector2DCurve* DI_Curve = NewObject<UNiagaraDataInterfaceVector2DCurve>();
	const TArray<FRichCurveKey> X_BaseKeys = FRichCurveKeyBP::KeysToBase(X_Keys);
	const TArray<FRichCurveKey> Y_BaseKeys = FRichCurveKeyBP::KeysToBase(Y_Keys);
	DI_Curve->XCurve.SetKeys(X_BaseKeys);
	DI_Curve->YCurve.SetKeys(Y_BaseKeys);
	return DI_Curve;
}

UNiagaraDataInterfaceVectorCurve* UFXConverterUtilitiesLibrary::CreateVec3CurveDI(
	TArray<FRichCurveKeyBP> X_Keys,
	TArray<FRichCurveKeyBP> Y_Keys,
	TArray<FRichCurveKeyBP> Z_Keys
	)
{
	UNiagaraDataInterfaceVectorCurve* DI_Curve = NewObject<UNiagaraDataInterfaceVectorCurve>();
	const TArray<FRichCurveKey> X_BaseKeys = FRichCurveKeyBP::KeysToBase(X_Keys);
	const TArray<FRichCurveKey> Y_BaseKeys = FRichCurveKeyBP::KeysToBase(Y_Keys);
	const TArray<FRichCurveKey> Z_BaseKeys = FRichCurveKeyBP::KeysToBase(Z_Keys);
	DI_Curve->XCurve.SetKeys(X_BaseKeys);
	DI_Curve->YCurve.SetKeys(Y_BaseKeys);
	DI_Curve->ZCurve.SetKeys(Z_BaseKeys);
	return DI_Curve;
}

UNiagaraDataInterfaceVector4Curve* UFXConverterUtilitiesLibrary::CreateVec4CurveDI(
	TArray<FRichCurveKeyBP> X_Keys,
	TArray<FRichCurveKeyBP> Y_Keys,
	TArray<FRichCurveKeyBP> Z_Keys,
	TArray<FRichCurveKeyBP> W_Keys
	)
{
	UNiagaraDataInterfaceVector4Curve* DI_Curve = NewObject<UNiagaraDataInterfaceVector4Curve>();
	const TArray<FRichCurveKey> X_BaseKeys = FRichCurveKeyBP::KeysToBase(X_Keys);
	const TArray<FRichCurveKey> Y_BaseKeys = FRichCurveKeyBP::KeysToBase(Y_Keys);
	const TArray<FRichCurveKey> Z_BaseKeys = FRichCurveKeyBP::KeysToBase(Z_Keys);
	const TArray<FRichCurveKey> W_BaseKeys = FRichCurveKeyBP::KeysToBase(W_Keys);
	DI_Curve->XCurve.SetKeys(X_BaseKeys);
	DI_Curve->YCurve.SetKeys(Y_BaseKeys);
	DI_Curve->ZCurve.SetKeys(Z_BaseKeys);
	DI_Curve->WCurve.SetKeys(W_BaseKeys);
	return DI_Curve;
}

UNiagaraSystemConversionContext* UFXConverterUtilitiesLibrary::CreateSystemConversionContext(UNiagaraSystem* InSystem)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
	FNiagaraSystemViewModelOptions SystemViewModelOptions = FNiagaraSystemViewModelOptions();
	SystemViewModelOptions.bCanAutoCompile = false;
	SystemViewModelOptions.bCanSimulate = false;
	SystemViewModelOptions.EditMode = ENiagaraSystemViewModelEditMode::SystemAsset;
	SystemViewModelOptions.MessageLogGuid = InSystem->GetAssetGuid();
	SystemViewModel->Initialize(*InSystem, SystemViewModelOptions);
	UNiagaraSystemConversionContext* SystemConversionContext = NewObject<UNiagaraSystemConversionContext>();
	SystemConversionContext->Init(InSystem, SystemViewModel);
	return SystemConversionContext;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleTypeDataGpuProps(UParticleModuleTypeDataGpu* ParticleModule)
{
	// empty impl, method arg taking UParticleModuleTypeDataGpu exposes this UObject type to python scripting reflection
	return;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleTypeDataBeam2Props(UParticleModuleTypeDataBeam2* ParticleModule)
{
	// empty impl, method arg taking UParticleModuleTypeDataBeam2 exposes this UObject type to python scripting reflection
	return;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleTypeDataMeshProps(
	UParticleModuleTypeDataMesh* ParticleModule
	, UStaticMesh*& OutMesh
	, float& OutLODSizeScale
	, bool& bOutUseStaticMeshLODs
	, bool& bOutCastShadows
	, bool& bOutDoCollisions
	, TEnumAsByte<EMeshScreenAlignment>& OutMeshAlignment
	, bool& bOutOverrideMaterial
	, bool& bOutOverrideDefaultMotionBlurSettings
	, bool& bOutEnableMotionBlur
	, UDistribution*& OutRollPitchYawRange
	, TEnumAsByte<EParticleAxisLock>& OutAxisLockOption
	, bool& bOutCameraFacing
	, TEnumAsByte<EMeshCameraFacingUpAxis>& OutCameraFacingUpAxisOption_DEPRECATED
	, TEnumAsByte<EMeshCameraFacingOptions>& OutCameraFacingOption
	, bool& bOutApplyParticleRotationAsSpin
	, bool& bOutFacingCameraDirectionRatherThanPosition
	, bool& bOutCollisionsConsiderParticleSize)
{
	OutMesh = ParticleModule->Mesh;
	OutLODSizeScale = ParticleModule->LODSizeScale;
	bOutUseStaticMeshLODs = ParticleModule->bUseStaticMeshLODs;
	bOutCastShadows = ParticleModule->CastShadows;
	bOutDoCollisions = ParticleModule->DoCollisions;
	OutMeshAlignment = ParticleModule->MeshAlignment;
	bOutOverrideMaterial = ParticleModule->bOverrideMaterial;
	bOutOverrideDefaultMotionBlurSettings = ParticleModule->bOverrideDefaultMotionBlurSettings;
	bOutEnableMotionBlur = ParticleModule->bEnableMotionBlur;
	OutRollPitchYawRange = ParticleModule->RollPitchYawRange.Distribution;
	OutAxisLockOption = ParticleModule->AxisLockOption;
	bOutCameraFacing = ParticleModule->bCameraFacing;
	OutCameraFacingUpAxisOption_DEPRECATED = ParticleModule->CameraFacingUpAxisOption_DEPRECATED;
	OutCameraFacingOption = ParticleModule->CameraFacingOption;
	bOutApplyParticleRotationAsSpin = ParticleModule->bApplyParticleRotationAsSpin;
	bOutFacingCameraDirectionRatherThanPosition = ParticleModule->bFaceCameraDirectionRatherThanPosition;
	bOutCollisionsConsiderParticleSize = ParticleModule->bCollisionsConsiderPartilceSize;
}

UClass* UFXConverterUtilitiesLibrary::GetParticleModuleTypeDataRibbonClass()
{
	return UParticleModuleTypeDataRibbon::StaticClass();
}

void UFXConverterUtilitiesLibrary::GetParticleModuleTypeDataRibbonProps(
	UParticleModuleTypeDataRibbon* ParticleModule
	, int32& OutMaxTessellationBetweenParticles
	, int32& OutSheetsPerTrail
	, int32& OutMaxTrailCount
	, int32& OutMaxParticleInTrailCount
	, bool& bOutDeadTrailsOnDeactivate
	, bool& bOutClipSourceSegment
	, bool& bOutEnablePreviousTangentRecalculation
	, bool& bOutTangentRecalculationEveryFrame
	, bool& bOutSpawnInitialParticle
	, TEnumAsByte<ETrailsRenderAxisOption>& OutRenderAxis
	, float& OutTangentSpawningScalar
	, bool& bOutRenderGeometry
	, bool& bOutRenderSpawnPoints
	, bool& bOutRenderTangents
	, bool& bOutRenderTessellation
	, float& OutTilingDistance
	, float& OutDistanceTessellationStepSize
	, bool& bOutEnableTangentDiffInterpScale
	, float& OutTangentTessellationScalar)
{
	OutMaxTessellationBetweenParticles = ParticleModule->MaxTessellationBetweenParticles;
	OutSheetsPerTrail = ParticleModule->SheetsPerTrail;
	OutMaxTrailCount = ParticleModule->MaxTrailCount;
	OutMaxParticleInTrailCount = ParticleModule->MaxParticleInTrailCount;
	bOutDeadTrailsOnDeactivate = ParticleModule->bDeadTrailsOnDeactivate;
	bOutClipSourceSegment = ParticleModule->bClipSourceSegement;
	bOutEnablePreviousTangentRecalculation = ParticleModule->bEnablePreviousTangentRecalculation;
	bOutTangentRecalculationEveryFrame = ParticleModule->bTangentRecalculationEveryFrame;
	bOutSpawnInitialParticle = ParticleModule->bSpawnInitialParticle;
	OutRenderAxis = ParticleModule->RenderAxis;
	OutTangentSpawningScalar = ParticleModule->TangentSpawningScalar;
	bOutRenderGeometry = ParticleModule->bRenderGeometry;
	bOutRenderSpawnPoints = ParticleModule->bRenderSpawnPoints;
	bOutRenderTangents = ParticleModule->bRenderTangents;
	bOutRenderTessellation = ParticleModule->bRenderTessellation;
	OutTilingDistance = ParticleModule->TilingDistance;
	OutDistanceTessellationStepSize = ParticleModule->DistanceTessellationStepSize;
	bOutEnableTangentDiffInterpScale = ParticleModule->bEnableTangentDiffInterpScale;
	OutTangentTessellationScalar = ParticleModule->TangentTessellationScalar;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSpawnProps(
	  UParticleModuleSpawn* ParticleModuleSpawn
	, UDistribution*& OutRate
	, UDistribution*& OutRateScale
	, TEnumAsByte<EParticleBurstMethod>& OutBurstMethod
	, TArray<FParticleBurstBlueprint>& OutBurstList
	, UDistribution*& OutBurstScale
	, bool& bOutApplyGlobalSpawnRateScale
	, bool& bOutProcessSpawnRate
	, bool& bOutProcessSpawnBurst)
{
	OutRate = ParticleModuleSpawn->Rate.Distribution;
	OutRateScale = ParticleModuleSpawn->RateScale.Distribution;
	OutBurstMethod = ParticleModuleSpawn->ParticleBurstMethod;
	OutBurstList = TArray<FParticleBurstBlueprint>(ParticleModuleSpawn->BurstList);
	OutBurstScale = ParticleModuleSpawn->BurstScale.Distribution;
	bOutApplyGlobalSpawnRateScale = ParticleModuleSpawn->bApplyGlobalSpawnRateScale;
	bOutProcessSpawnRate = ParticleModuleSpawn->bProcessSpawnRate;
	bOutProcessSpawnBurst = ParticleModuleSpawn->bProcessBurstList;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSpawnPerUnitProps(
	UParticleModuleSpawnPerUnit* ParticleModule
	, float& OutUnitScalar
	, float& OutMovementTolerance
	, UDistribution*& OutSpawnPerUnit
	, float& OutMaxFrameDistance
	, bool& bOutIgnoreSpawnRateWhenMoving
	, bool& bOutIgnoreMovementAlongX
	, bool& bOutIgnoreMovementAlongY
	, bool& bOutIgnoreMovementAlongZ
	, bool& bOutProcessSpawnRate
	, bool& bOutProcessBurstList)
{
	OutUnitScalar = ParticleModule->UnitScalar;
	OutMovementTolerance = ParticleModule->MovementTolerance;
	OutSpawnPerUnit = ParticleModule->SpawnPerUnit.Distribution;
	OutMaxFrameDistance = ParticleModule->MaxFrameDistance;
	bOutIgnoreSpawnRateWhenMoving = ParticleModule->bIgnoreSpawnRateWhenMoving;
	bOutIgnoreMovementAlongX = ParticleModule->bIgnoreMovementAlongX;
	bOutIgnoreMovementAlongY = ParticleModule->bIgnoreMovementAlongY;
	bOutIgnoreMovementAlongZ = ParticleModule->bIgnoreMovementAlongZ;
	bOutProcessSpawnRate = ParticleModule->bProcessSpawnRate;
	bOutProcessBurstList = ParticleModule->bProcessBurstList;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleRequiredPerRendererProps(
	UParticleModuleRequired* ParticleModuleRequired
	, UMaterialInterface*& OutMaterialInterface
	, TEnumAsByte<EParticleScreenAlignment>& OutScreenAlignment
	, int32& OutSubImages_Horizontal
	, int32& OutSubImages_Vertical
	, TEnumAsByte<EParticleSortMode>& OutSortMode
	, TEnumAsByte<EParticleSubUVInterpMethod>& OutInterpolationMethod
	, uint8& bOutRemoveHMDRoll
	, float& OutMinFacingCameraBlendDistance
	, float& OutMaxFacingCameraBlendDistance
	, UTexture2D*& OutCutoutTexture
	, TEnumAsByte<ESubUVBoundingVertexCount>& OutBoundingMode
	, TEnumAsByte<EOpacitySourceMode>& OutOpacitySourceMode
	, TEnumAsByte< EEmitterNormalsMode>& OutEmitterNormalsMode
	, float& OutAlphaThreshold)
{
	OutMaterialInterface = ParticleModuleRequired->Material;
	OutScreenAlignment = ParticleModuleRequired->ScreenAlignment;
	OutSubImages_Horizontal = ParticleModuleRequired->SubImages_Horizontal;
	OutSubImages_Vertical = ParticleModuleRequired->SubImages_Vertical;
	OutSortMode = ParticleModuleRequired->SortMode;
	OutInterpolationMethod = ParticleModuleRequired->InterpolationMethod;
	bOutRemoveHMDRoll = ParticleModuleRequired->bRemoveHMDRoll;
	OutMinFacingCameraBlendDistance = ParticleModuleRequired->MinFacingCameraBlendDistance;
	OutMaxFacingCameraBlendDistance = ParticleModuleRequired->MaxFacingCameraBlendDistance;
	OutCutoutTexture = ParticleModuleRequired->CutoutTexture;
	OutBoundingMode = ParticleModuleRequired->BoundingMode;
	OutOpacitySourceMode = ParticleModuleRequired->OpacitySourceMode;
	OutAlphaThreshold = ParticleModuleRequired->AlphaThreshold;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleRequiredPerModuleProps(
	UParticleModuleRequired* ParticleModuleRequired
	, bool& bOutOrbitModuleAffectsVelocityAlignment
	, float& OutRandomImageTime
	, int32& OutRandomImageChanges
	, bool& bOutOverrideSystemMacroUV
	, FVector& OutMacroUVPosition
	, float& OutMacroUVRadius
	)
{
	bOutOrbitModuleAffectsVelocityAlignment = ParticleModuleRequired->bOrbitModuleAffectsVelocityAlignment;
	OutRandomImageTime = ParticleModuleRequired->RandomImageTime;
	OutRandomImageChanges = ParticleModuleRequired->RandomImageChanges;
	bOutOverrideSystemMacroUV = ParticleModuleRequired->bOverrideSystemMacroUV;
	OutMacroUVPosition = ParticleModuleRequired->MacroUVPosition;
	OutMacroUVRadius = ParticleModuleRequired->MacroUVRadius;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleRequiredPerEmitterProps(
	UParticleModuleRequired* ParticleModuleRequired
	, FVector& OutEmitterOrigin
	, FRotator& OutEmitterRotation
	, bool& bOutUseLocalSpace
	, bool& bOutKillOnDeactivate
	, bool& bOutKillOnCompleted
	, bool& bOutUseLegacyEmitterTime
	, bool& bOutEmitterDurationUseRange
	, float& OutEmitterDuration
	, float& OutEmitterDurationLow
	, bool& bOUtEmitterDelayUseRange
	, bool& bOutDelayFirstLoopOnly
	, float& OutEmitterDelay
	, float& OutEmitterDelayLow
	, bool& bOutDurationRecalcEachLoop
	, int32& OutEmitterLoops)
{
	OutEmitterOrigin = ParticleModuleRequired->EmitterOrigin;
	OutEmitterRotation = ParticleModuleRequired->EmitterRotation;
	bOutUseLocalSpace = ParticleModuleRequired->bUseLocalSpace;
	bOutKillOnDeactivate = ParticleModuleRequired->bKillOnDeactivate;
	bOutKillOnCompleted = ParticleModuleRequired->bKillOnCompleted;
	bOutUseLegacyEmitterTime = ParticleModuleRequired->bUseLegacyEmitterTime;
	bOutEmitterDurationUseRange = ParticleModuleRequired->bEmitterDurationUseRange;
	OutEmitterDuration = ParticleModuleRequired->EmitterDuration;
	OutEmitterDurationLow = ParticleModuleRequired->EmitterDurationLow;
	bOUtEmitterDelayUseRange = ParticleModuleRequired->bEmitterDelayUseRange;
	bOutDelayFirstLoopOnly = ParticleModuleRequired->bDelayFirstLoopOnly;
	OutEmitterDelay = ParticleModuleRequired->EmitterDelay;
	OutEmitterDelayLow = ParticleModuleRequired->EmitterDelayLow;
	bOutDurationRecalcEachLoop = ParticleModuleRequired->bDurationRecalcEachLoop;
	OutEmitterLoops = ParticleModuleRequired->EmitterLoops;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleColorProps(UParticleModuleColor* ParticleModule, UDistribution*& OutStartColor, UDistribution*& OutStartAlpha, bool& bOutClampAlpha)
{
	OutStartColor = ParticleModule->StartColor.Distribution;
	OutStartAlpha = ParticleModule->StartAlpha.Distribution;
	bOutClampAlpha = ParticleModule->bClampAlpha;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleColorOverLifeProps(UParticleModuleColorOverLife* ParticleModule, UDistribution*& OutColorOverLife, UDistribution*& OutAlphaOverLife, bool& bOutClampAlpha)
{
	OutColorOverLife = ParticleModule->ColorOverLife.Distribution;
	OutAlphaOverLife = ParticleModule->AlphaOverLife.Distribution;
	bOutClampAlpha = ParticleModule->bClampAlpha;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLifetimeProps(UParticleModuleLifetime* ParticleModule, UDistribution*& OutLifetime)
{
	OutLifetime = ParticleModule->Lifetime.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSizeProps(UParticleModuleSize* ParticleModule, UDistribution*& OutStartSize)
{
	OutStartSize = ParticleModule->StartSize.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleVelocityProps(UParticleModuleVelocity* ParticleModule, UDistribution*& OutStartVelocity, UDistribution*& OutStartVelocityRadial, bool& bOutInWorldSpace, bool& bOutApplyOwnerScale)
{
	OutStartVelocity = ParticleModule->StartVelocity.Distribution;
	OutStartVelocityRadial = ParticleModule->StartVelocityRadial.Distribution;
	bOutInWorldSpace = ParticleModule->bInWorldSpace;
	bOutApplyOwnerScale = ParticleModule->bApplyOwnerScale;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleVelocityConeProps(UParticleModuleVelocityCone* ParticleModule, UDistribution*& OutAngleDistribution, UDistribution*& OutVelocityDistribution, FVector& OutDirection, bool& bOutWorldSpace, bool& bOutApplyOwnerScale)
{
	OutAngleDistribution	= ParticleModule->Angle.Distribution;
	OutVelocityDistribution	= ParticleModule->Velocity.Distribution;
	OutDirection			= ParticleModule->Direction;
	bOutWorldSpace			= ParticleModule->bInWorldSpace;
	bOutApplyOwnerScale		= ParticleModule->bApplyOwnerScale;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleVelocityOverLifetimeProps(
	UParticleModuleVelocityOverLifetime* ParticleModule
	, UDistribution*& OutVelOverLife
	, bool& bOutAbsolute
	, bool& bOutInWorldSpace
	, bool& bOutApplyOwnerScale)
{
	OutVelOverLife = ParticleModule->VelOverLife.Distribution;
	bOutAbsolute = ParticleModule->Absolute;
	bOutInWorldSpace = ParticleModule->bInWorldSpace;
	bOutApplyOwnerScale = ParticleModule->bApplyOwnerScale;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleConstantAccelerationProps(UParticleModuleAccelerationConstant* ParticleModule, FVector& OutConstAcceleration)
{
	OutConstAcceleration = ParticleModule->Acceleration;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationPrimitiveSphereProps(
	UParticleModuleLocationPrimitiveSphere* ParticleModule
	, UDistribution*& OutStartRadius
	, bool& bOutPositiveX
	, bool& bOutPositiveY
	, bool& bOutPositiveZ
	, bool& bOutNegativeX
	, bool& bOutNegativeY
	, bool& bOutNegativeZ
	, bool& bOutSurfaceOnly
	, bool& bOutVelocity
	, UDistribution*& OutVelocityScale
	, UDistribution*& OutStartLocation)
{
	OutStartRadius = ParticleModule->StartRadius.Distribution;
	UFXConverterUtilitiesLibrary::GetParticleModuleLocationPrimitiveBaseProps(
		ParticleModule, bOutPositiveX, bOutPositiveY, bOutPositiveZ
		, bOutNegativeX, bOutNegativeY, bOutNegativeZ, bOutSurfaceOnly
		, bOutVelocity, OutVelocityScale, OutStartLocation
	);
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationPrimitiveCylinderProps(
	UParticleModuleLocationPrimitiveCylinder* ParticleModule
	, bool& bOutRadialVelocity
	, UDistribution*& OutStartRadius
	, UDistribution*& OutStartHeight
	, TEnumAsByte<CylinderHeightAxis>& OutHeightAxis
	, bool& bOutPositiveX
	, bool& bOutPositiveY
	, bool& bOutPositiveZ
	, bool& bOutNegativeX
	, bool& bOutNegativeY
	, bool& bOutNegativeZ
	, bool& bOutSurfaceOnly
	, bool& bOutVelocity
	, UDistribution*& OutVelocityScale
	, UDistribution*& OutStartLocation)
{
	bOutRadialVelocity = ParticleModule->RadialVelocity;
	OutStartRadius = ParticleModule->StartRadius.Distribution;
	OutStartHeight = ParticleModule->StartHeight.Distribution;
	OutHeightAxis = ParticleModule->HeightAxis;
	UFXConverterUtilitiesLibrary::GetParticleModuleLocationPrimitiveBaseProps(
		ParticleModule, bOutPositiveX, bOutPositiveY, bOutPositiveZ
		, bOutNegativeX, bOutNegativeY, bOutNegativeZ, bOutSurfaceOnly
		, bOutVelocity, OutVelocityScale, OutStartLocation
	);
}

void UFXConverterUtilitiesLibrary::GetParticleModuleVelocityInheritParentProps(
	UParticleModuleVelocityInheritParent* ParticleModule
	, UDistribution*& OutScale
	, bool& bOutInWorldSpace
	, bool& bOutApplyOwnerScale)
{
	OutScale = ParticleModule->Scale.Distribution;
	bOutInWorldSpace = ParticleModule->bInWorldSpace;
	bOutApplyOwnerScale = ParticleModule->bApplyOwnerScale;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleOrientationAxisLockProps(UParticleModuleOrientationAxisLock* ParticleModule, TEnumAsByte<EParticleAxisLock>& OutLockAxisFlags)
{
	OutLockAxisFlags = ParticleModule->LockAxisFlags;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleMeshRotationProps(UParticleModuleMeshRotation* ParticleModule, UDistribution*& OutStartRotation, bool& bOutInheritParentRotation)
{
	OutStartRotation = ParticleModule->StartRotation.Distribution;
	bOutInheritParentRotation = ParticleModule->bInheritParent;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleCollisionProps(
	  UParticleModuleCollision* ParticleModule
	, UDistribution*& OutDampingFactor
	, UDistribution*& OutDampingFactorRotation
	, UDistribution*& OutMaxCollisions
	, TEnumAsByte<EParticleCollisionComplete>& OutCollisionCompleteOption
	, TArray<TEnumAsByte<EObjectTypeQuery>>& OutCollisionTypes
	, bool& bOutApplyPhysics
	, bool& bOutIgnoreTriggerVolumes
	, UDistribution*& OutParticleMass
	, float& OutDirScalar
	, bool& bOutPawnsDoNotDecrementCount
	, bool& bOutOnlyVerticalNormalsDecrementCount
	, float& OutVerticalFudgeFactor
	, UDistribution*& OutDelayAmount
	, bool& bOutDropDetail
	, bool& bOutCollideOnlyIfVisible
	, bool& bOutIgnoreSourceActor
	, float& OutMaxCollisionDistance)
{
	OutDampingFactor = ParticleModule->DampingFactor.Distribution;
	OutDampingFactorRotation = ParticleModule->DampingFactorRotation.Distribution;
	OutMaxCollisions = ParticleModule->MaxCollisions.Distribution;
	OutCollisionCompleteOption = ParticleModule->CollisionCompletionOption;
	OutCollisionTypes = ParticleModule->CollisionTypes;
	bOutApplyPhysics = ParticleModule->bApplyPhysics;
	bOutIgnoreTriggerVolumes = ParticleModule->bIgnoreTriggerVolumes;
	OutParticleMass = ParticleModule->ParticleMass.Distribution;
	OutDirScalar = ParticleModule->DirScalar;
	bOutPawnsDoNotDecrementCount = ParticleModule->bPawnsDoNotDecrementCount;
	bOutOnlyVerticalNormalsDecrementCount = ParticleModule->bOnlyVerticalNormalsDecrementCount;
	OutVerticalFudgeFactor = ParticleModule->VerticalFudgeFactor;
	OutDelayAmount = ParticleModule->DelayAmount.Distribution;
	bOutDropDetail = false;
	bOutCollideOnlyIfVisible = ParticleModule->bCollideOnlyIfVisible;
	bOutIgnoreSourceActor = ParticleModule->bIgnoreSourceActor;
	OutMaxCollisionDistance = ParticleModule->MaxCollisionDistance;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSizeScaleProps(UParticleModuleSizeScale* ParticleModule, UDistribution*& OutSizeScale)
{
	OutSizeScale = ParticleModule->SizeScale.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSizeScaleBySpeedProps(UParticleModuleSizeScaleBySpeed* ParticleModule, FVector2D& OutSpeedScale, FVector2D& OutMaxScale)
{
	OutSpeedScale = ParticleModule->SpeedScale;
	OutMaxScale = ParticleModule->MaxScale;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleVectorFieldLocalProps(
	  UParticleModuleVectorFieldLocal* ParticleModule
	, UVectorField* OutVectorField
	, FVector& OutRelativeTranslation
	, FRotator& OutRelativeRotation
	, FVector& OutRelativeScale3D
	, float& OutIntensity
	, float& OutTightness
	, bool& bOutIgnoreComponentTransform
	, bool& bOutTileX
	, bool& bOutTileY
	, bool& bOutTileZ
	, bool& bOutUseFixDT)
{
	OutVectorField = ParticleModule->VectorField;
	OutRelativeTranslation = ParticleModule->RelativeTranslation;
	OutRelativeRotation = ParticleModule->RelativeRotation;
	OutRelativeScale3D = ParticleModule->RelativeScale3D;
	OutIntensity = ParticleModule->Intensity;
	OutTightness = ParticleModule->Tightness;
	bOutIgnoreComponentTransform = ParticleModule->bIgnoreComponentTransform;
	bOutTileX = ParticleModule->bTileX;
	bOutTileY = ParticleModule->bTileY;
	bOutTileZ = ParticleModule->bTileZ;
	bOutUseFixDT = ParticleModule->bUseFixDT;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleVectorFieldRotationRateProps(UParticleModuleVectorFieldRotationRate* ParticleModule, FVector& OutRotationRate)
{
	OutRotationRate = ParticleModule->RotationRate;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleOrbitProps(
	  UParticleModuleOrbit* ParticleModule
	, TEnumAsByte<enum EOrbitChainMode>& OutChainMode
	, UDistribution*& OutOffsetAmount
	, FOrbitOptionsBP& OutOffsetOptions
	, UDistribution*& OutRotationAmount
	, FOrbitOptionsBP& OutRotationOptions
	, UDistribution*& OutRotationRateAmount
	, FOrbitOptionsBP& OutRotationRateOptions)
{
	OutChainMode = ParticleModule->ChainMode;
	OutOffsetAmount = ParticleModule->OffsetAmount.Distribution;
	OutOffsetOptions = ParticleModule->OffsetOptions;
	OutRotationAmount = ParticleModule->RotationAmount.Distribution;
	OutRotationOptions = ParticleModule->RotationOptions;
	OutRotationRateAmount = ParticleModule->RotationRateAmount.Distribution;
	OutRotationRateOptions = ParticleModule->RotationRateOptions;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSizeMultiplyLifeProps(
	UParticleModuleSizeMultiplyLife* ParticleModule
	, UDistribution*& OutLifeMultiplier
	, bool& OutMultiplyX
	, bool& OutMultiplyY
	, bool& OutMultiplyZ)
{
	OutLifeMultiplier = ParticleModule->LifeMultiplier.Distribution;
	OutMultiplyX = ParticleModule->MultiplyX;
	OutMultiplyY = ParticleModule->MultiplyY;
	OutMultiplyZ = ParticleModule->MultiplyZ;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleColorScaleOverLifeProps(
	UParticleModuleColorScaleOverLife* ParticleModule
	, UDistribution*& OutColorScaleOverLife
	, UDistribution*& OutAlphaScaleOverLife
	, bool& bOutEmitterTime)
{
	OutColorScaleOverLife = ParticleModule->ColorScaleOverLife.Distribution;
	OutAlphaScaleOverLife = ParticleModule->AlphaScaleOverLife.Distribution;
	bOutEmitterTime = ParticleModule->bEmitterTime;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleRotationProps(UParticleModuleRotation* ParticleModule, UDistribution*& OutStartRotation)
{
	OutStartRotation = ParticleModule->StartRotation.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleRotationRateProps(UParticleModuleRotationRate* ParticleModule, UDistribution*& OutStartRotationRate)
{
	OutStartRotationRate = ParticleModule->StartRotationRate.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleRotationRateMultiplyLifeProps(UParticleModuleRotationRateMultiplyLife* ParticleModule, UDistribution*& OutLifeMultiplier)
{
	OutLifeMultiplier = ParticleModule->LifeMultiplier.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleMeshRotationRateProps(UParticleModuleMeshRotationRate* ParticleModule, UDistribution*& OutStartRotationRate)
{
	OutStartRotationRate = ParticleModule->StartRotationRate.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleRotationOverLifetimeProps(UParticleModuleRotationOverLifetime* ParticleModule, UDistribution*& OutRotationOverLife, bool& bOutScale)
{
	OutRotationOverLife = ParticleModule->RotationOverLife.Distribution;
	bOutScale = ParticleModule->Scale;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleMeshRotationRateMultiplyLifeProps(UParticleModuleMeshRotationRateMultiplyLife* ParticleModule, UDistribution*& OutLifeMultiplier)
{
	OutLifeMultiplier = ParticleModule->LifeMultiplier.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleMeshRotationRateOverLifeProps(UParticleModuleMeshRotationRateOverLife* ParticleModule, UDistribution*& OutRotRate, bool& bOutScaleRotRate)
{
	OutRotRate = ParticleModule->RotRate.Distribution;
	bOutScaleRotRate = ParticleModule->bScaleRotRate;
}

void UFXConverterUtilitiesLibrary::GetParticleModulePivotOffsetProps(UParticleModulePivotOffset* ParticleModule, FVector2D& OutPivotOffset)
{
	OutPivotOffset = ParticleModule->PivotOffset;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSubUVProps(
	UParticleModuleSubUV* ParticleModule
	, USubUVAnimation*& OutAnimation
	, UDistribution*& OutSubImageIndex
	, bool& bOutUseRealTime)
{
	OutAnimation = ParticleModule->Animation;
	OutSubImageIndex = ParticleModule->SubImageIndex.Distribution;
	bOutUseRealTime = ParticleModule->bUseRealTime;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleCameraOffsetProps(
	UParticleModuleCameraOffset* ParticleModule
	, UDistribution*& OutCameraOffset
	, bool& bOutSpawnTimeOnly
	, TEnumAsByte<EParticleCameraOffsetUpdateMethod>& OutUpdateMethod)
{
	OutCameraOffset = ParticleModule->CameraOffset.Distribution;
	bOutSpawnTimeOnly = ParticleModule->bSpawnTimeOnly;
	OutUpdateMethod = ParticleModule->UpdateMethod;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleSubUVMovieProps(
	UParticleModuleSubUVMovie* ParticleModule
	, bool& bOutUseEmitterTime
	, UDistribution*& OutFrameRate
	, int32& OutStartingFrame)
{
	bOutUseEmitterTime = ParticleModule->bUseEmitterTime;
	OutFrameRate = ParticleModule->FrameRate.Distribution;
	OutStartingFrame = ParticleModule->StartingFrame;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleParameterDynamicProps(UParticleModuleParameterDynamic* ParticleModule, TArray<FEmitterDynamicParameterBP>& OutDynamicParams, bool& bOutUsesVelocity)
{	
	OutDynamicParams.Reserve(ParticleModule->DynamicParams.Num());
	for (const FEmitterDynamicParameter& DynamicParam : ParticleModule->DynamicParams)
	{
		OutDynamicParams.Add(DynamicParam);
	}
	bOutUsesVelocity = ParticleModule->bUsesVelocity;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleAccelerationDragProps(UParticleModuleAccelerationDrag* ParticleModule, UDistribution*& OutDragCoefficientRaw)
{
	OutDragCoefficientRaw = ParticleModule->DragCoefficientRaw.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleAccelerationDragScaleOverLifeProps(UParticleModuleAccelerationDragScaleOverLife* ParticleModule, UDistribution*& OutDragScaleRaw)
{
	OutDragScaleRaw = ParticleModule->DragScaleRaw.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleAccelerationProps(UParticleModuleAcceleration* ParticleModule, UDistribution*& OutAcceleration, bool& bOutApplyOwnerScale)
{
	OutAcceleration = ParticleModule->Acceleration.Distribution;
	bOutApplyOwnerScale = ParticleModule->bApplyOwnerScale;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleAccelerationOverLifetimeProps(UParticleModuleAccelerationOverLifetime* ParticleModule, UDistribution*& OutAccelOverLife)
{
	OutAccelOverLife = ParticleModule->AccelOverLife.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleTrailSourceProps(
	UParticleModuleTrailSource* ParticleModule
	, TEnumAsByte<ETrail2SourceMethod>& OutSourceMethod
	, FName& OutSourceName
	, UDistribution*& OutSourceStrength
	, bool& bOutLockSourceStrength
	, int32& OutSourceOffsetCount
	, TArray<FVector>& OutSourceOffsetDefaults
	, TEnumAsByte<EParticleSourceSelectionMethod>& OutSelectionMethod
	, bool& bOutInheritRotation)
{
	OutSourceMethod = ParticleModule->SourceMethod;
	OutSourceName = ParticleModule->SourceName;
	OutSourceStrength = ParticleModule->SourceStrength.Distribution;
	bOutLockSourceStrength = ParticleModule->bLockSourceStength;
	OutSourceOffsetCount = ParticleModule->SourceOffsetCount;
	OutSourceOffsetDefaults = ParticleModule->SourceOffsetDefaults;
	OutSelectionMethod = ParticleModule->SelectionMethod;
	bOutInheritRotation = ParticleModule->bInheritRotation;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleAttractorParticleProps(
	UParticleModuleAttractorParticle* ParticleModule
	, FName& OutEmitterName
	, UDistribution*& OutRange
	, bool& bOutStrengthByDistance
	, UDistribution*& OutStrength
	, bool& bOutAffectBaseVelocity
	, TEnumAsByte<EAttractorParticleSelectionMethod>& OutSelectionMethod
	, bool& bOutRenewSource
	, bool& bOutInheritSourceVelocity)
{
	OutEmitterName = ParticleModule->EmitterName;
	OutRange = ParticleModule->Range.Distribution;
	bOutStrengthByDistance = ParticleModule->bStrengthByDistance;
	OutStrength = ParticleModule->Strength.Distribution;
	bOutAffectBaseVelocity = ParticleModule->bAffectBaseVelocity;
	OutSelectionMethod = ParticleModule->SelectionMethod;
	bOutRenewSource = ParticleModule->bRenewSource;
	bOutInheritSourceVelocity = ParticleModule->bInheritSourceVel;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleAttractorPointProps(
	UParticleModuleAttractorPoint* ParticleModule
	, UDistribution*& OutPosition
	, UDistribution*& OutRange
	, UDistribution*& OutStrength
	, bool& boutStrengthByDistance
	, bool& bOutAffectsBaseVelocity
	, bool& bOutOverrideVelocity
	, bool& bOutUseWorldSpacePosition
	, bool& bOutPositiveX
	, bool& bOutPositiveY
	, bool& bOutPositiveZ
	, bool& bOutNegativeX
	, bool& bOutNegativeY
	, bool& bOutNegativeZ)
{
	OutPosition = ParticleModule->Position.Distribution;
	OutRange = ParticleModule->Range.Distribution;
	OutStrength = ParticleModule->Strength.Distribution;
	boutStrengthByDistance = ParticleModule->StrengthByDistance;
	bOutAffectsBaseVelocity = ParticleModule->bAffectBaseVelocity;
	bOutOverrideVelocity = ParticleModule->bOverrideVelocity;
	bOutUseWorldSpacePosition = ParticleModule->bUseWorldSpacePosition;
	bOutPositiveX = ParticleModule->Positive_X;
	bOutPositiveY = ParticleModule->Positive_Y;
	bOutPositiveZ = ParticleModule->Positive_Z;
	bOutNegativeX = ParticleModule->Negative_X;
	bOutNegativeY = ParticleModule->Negative_Y;
	bOutNegativeZ = ParticleModule->Negative_Z;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleAttractorLineProps(
	UParticleModuleAttractorLine* ParticleModule
	, FVector& OutStartPoint
	, FVector& OutEndPoint
	, UDistribution*& OutRange
	, UDistribution*& OutStrength)
{
	OutStartPoint = ParticleModule->EndPoint0;
	OutEndPoint = ParticleModule->EndPoint1;
	OutRange = ParticleModule->Range.Distribution;
	OutStrength = ParticleModule->Strength.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationDirectProps(
	UParticleModuleLocationDirect* ParticleModule
	, UDistribution*& OutLocation
	, UDistribution*& OutLocationOffset
	, UDistribution*& OutScaleFactor)
{
	OutLocation = ParticleModule->Location.Distribution;
	OutLocationOffset = ParticleModule->LocationOffset.Distribution;
	OutScaleFactor = ParticleModule->ScaleFactor.Distribution;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationProps(
	UParticleModuleLocation* ParticleModule
	, UDistribution*& OutStartLocation
	, float& OutDistributeOverNPoints
	, float& OutDistributeThreshold)
{
	OutStartLocation = ParticleModule->StartLocation.Distribution;
	OutDistributeOverNPoints = ParticleModule->DistributeOverNPoints;
	OutDistributeThreshold = ParticleModule->DistributeThreshold;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationBoneSocketProps(
	UParticleModuleLocationBoneSocket* ParticleModule
	, TEnumAsByte<ELocationBoneSocketSource>& OutSourceType
	, FVector& OutUniversalOffset
	, TArray<FLocationBoneSocketInfoBP>& OutSourceLocations
	, TEnumAsByte<ELocationBoneSocketSelectionMethod>& OutSelectionMethod
	, bool& bOutUpdatePositionEachFrame
	, bool& bOutOrientMeshEmitters
	, bool& bOutInheritBoneVelocity
	, float& OutInheritVelocityScale
	, FName& OutSkelMeshActorParamName
	, int32& OutNumPreSelectedIndices
	, USkeletalMesh*& OutEditorSkelMesh)
{
	OutSourceType = ParticleModule->SourceType;
	OutUniversalOffset = ParticleModule->UniversalOffset;
	for (const FLocationBoneSocketInfo& SourceLocation : ParticleModule->SourceLocations)
	{
		OutSourceLocations.Emplace(FLocationBoneSocketInfoBP(SourceLocation));
	}
	OutSelectionMethod = ParticleModule->SelectionMethod;
	bOutUpdatePositionEachFrame = ParticleModule->bUpdatePositionEachFrame;
	bOutOrientMeshEmitters = ParticleModule->bOrientMeshEmitters;
	bOutInheritBoneVelocity = ParticleModule->bInheritBoneVelocity;
	OutInheritVelocityScale = ParticleModule->InheritVelocityScale;
	OutSkelMeshActorParamName = ParticleModule->SkelMeshActorParamName;
	OutNumPreSelectedIndices = ParticleModule->NumPreSelectedIndices;
	OutEditorSkelMesh = ParticleModule->EditorSkelMesh;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationWorldOffsetProps(
	UParticleModuleLocationWorldOffset* ParticleModule
	, UDistribution*& OutStartLocation
	, float& OutDistributeOverNPoints
	, float& OutDistributeThreshold
)
{
	GetParticleModuleLocationProps(ParticleModule
		, OutStartLocation
		, OutDistributeOverNPoints
		, OutDistributeThreshold);
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLocationSkelVertSurfaceProps(
	UParticleModuleLocationSkelVertSurface* ParticleModule
	, TEnumAsByte<enum ELocationSkelVertSurfaceSource>& OutSourceType
	, FVector& OutUniversalOffset
	, bool& bOutUpdatePositionEachFrame
	, bool& bOutOrientMeshEmitters
	, bool& bOutInheritBoneVelocity
	, float& OutInheritVelocityScale
	, FName& OutSkelMeshActorParamName
	, USkeletalMesh*& OutEditorSkelMesh
	, TArray<FName>& OutValidAssociatedBones
	, bool& bOutEnforceNormalCheck
	, FVector& OutNormalToCompare
	, float& OutNormalCheckToleranceDegrees
	, float& OutNormalCheckTolerance
	, TArray<int32>& OutValidMaterialIndices
	, bool& bOutInheritVertexColor
	, bool& bOutInheritUV
	, int32& OutInheritUVChannel)
{
	OutSourceType = ParticleModule->SourceType;
	OutUniversalOffset = ParticleModule->UniversalOffset;
	bOutUpdatePositionEachFrame = ParticleModule->bUpdatePositionEachFrame;
	bOutOrientMeshEmitters = ParticleModule->bOrientMeshEmitters;
	bOutInheritBoneVelocity = ParticleModule->bInheritBoneVelocity;
	OutInheritVelocityScale = ParticleModule->InheritVelocityScale;
	OutSkelMeshActorParamName = ParticleModule->SkelMeshActorParamName;
	OutEditorSkelMesh = ParticleModule->EditorSkelMesh;
	OutValidAssociatedBones = ParticleModule->ValidAssociatedBones;
	bOutEnforceNormalCheck = ParticleModule->bEnforceNormalCheck;
	OutNormalToCompare = ParticleModule->NormalToCompare;
	OutNormalCheckToleranceDegrees = ParticleModule->NormalCheckToleranceDegrees;
	OutNormalCheckTolerance = ParticleModule->NormalCheckTolerance;
	OutValidMaterialIndices = ParticleModule->ValidMaterialIndices;
	bOutInheritVertexColor = ParticleModule->bInheritVertexColor;
	bOutInheritUV = ParticleModule->bInheritUV;
	OutInheritUVChannel = (int32)ParticleModule->InheritUVChannel;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleKillBoxProps(
	UParticleModuleKillBox* ParticleModule
	, UDistribution*& OutLowerLeftCorner
	, UDistribution*& OutUpperRightCorner
	, bool& bOutWorldSpaceCoords
	, bool& bOutKillInside
	, bool& bOutAxisAlignedAndFixedSize)
{
	OutLowerLeftCorner = ParticleModule->LowerLeftCorner.Distribution;
	OutUpperRightCorner = ParticleModule->UpperRightCorner.Distribution;
	bOutWorldSpaceCoords = ParticleModule->bAbsolute;
	bOutKillInside = ParticleModule->bKillInside;
	bOutAxisAlignedAndFixedSize = ParticleModule->bAxisAlignedAndFixedSize;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleLightProps(
	UParticleModuleLight* ParticleModule
	, bool& bOutUseInverseSquaredFalloff
	, bool& bOutAffectsTranslucency
	, bool& bOutPreviewLightRadius
	, float& OutSpawnFraction
	, UDistribution*& OutColorScaleOverLife
	, UDistribution*& OutBrightnessOverLife
	, UDistribution*& OutRadiusScale
	, UDistribution*& OutLightExponent
	, FLightingChannels& OutLightingChannels
	, float& OutVolumetricScatteringIntensity
	, bool& bOutHighQualityLights
	, bool& bOutShadowCastingLights)
{
	bOutUseInverseSquaredFalloff = ParticleModule->bUseInverseSquaredFalloff;
	bOutAffectsTranslucency = ParticleModule->bAffectsTranslucency;
	bOutPreviewLightRadius = ParticleModule->bPreviewLightRadius;
	OutSpawnFraction = ParticleModule->SpawnFraction;
	OutColorScaleOverLife = ParticleModule->ColorScaleOverLife.Distribution;
	OutBrightnessOverLife = ParticleModule->BrightnessOverLife.Distribution;
	OutRadiusScale = ParticleModule->RadiusScale.Distribution;
	OutLightExponent = ParticleModule->LightExponent.Distribution;
	OutLightingChannels = ParticleModule->LightingChannels;
	OutVolumetricScatteringIntensity = ParticleModule->VolumetricScatteringIntensity;
	bOutHighQualityLights = ParticleModule->bHighQualityLights;
	bOutShadowCastingLights = ParticleModule->bShadowCastingLights;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleMeshMaterialProps(UParticleModuleMeshMaterial* ParticleModule, TArray<UMaterialInterface*>& OutMeshMaterials)
{
	OutMeshMaterials = ParticleModule->MeshMaterials;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleBeamSourceProps(UParticleModuleBeamSource* ParticleModule, UDistribution*& OutSource, bool& bOutSourceAbsolute)
{
	OutSource = ParticleModule->Source.Distribution;
	bOutSourceAbsolute = ParticleModule->bSourceAbsolute;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleBeamTargetProps(UParticleModuleBeamTarget* ParticleModule, UDistribution*& OutTarget, bool& bOutTargetAbsolute)
{
	OutTarget = ParticleModule->Target.Distribution;
	bOutTargetAbsolute = ParticleModule->bTargetAbsolute;
}

void UFXConverterUtilitiesLibrary::GetParticleModuleBeamNoiseProps(UParticleModuleBeamNoise* ParticleModule)
{
}

void UFXConverterUtilitiesLibrary::SetMeshRendererMaterialOverridesFromCascade(UNiagaraMeshRendererProperties* MeshRendererProps, TArray<UMaterialInterface*> MeshMaterials)
{
	for (UMaterialInterface* MeshMaterial : MeshMaterials)
	{
		FNiagaraMeshMaterialOverride& MeshMaterialOverride = MeshRendererProps->OverrideMaterials.Add_GetRef(FNiagaraMeshMaterialOverride());
		MeshMaterialOverride.ExplicitMat = MeshMaterial;
	}
}

void UFXConverterUtilitiesLibrary::GetDistributionMinMaxValues(
	  UDistribution* Distribution
	, bool& bOutSuccess
	, FVector& OutMinValue
	, FVector& OutMaxValue)
{
	bOutSuccess = false;
	OutMinValue = FVector::ZeroVector;
	OutMaxValue = FVector::ZeroVector;

	if (Distribution == nullptr)
	{
		return;
	}

	if (Distribution->IsA<UDistributionFloatConstant>())
	{	
		float DistributionValue = 0.0f;
		GetFloatDistributionConstValues(static_cast<UDistributionFloatConstant*>(Distribution), DistributionValue);
		bOutSuccess = true;
		OutMinValue = FVector(DistributionValue, 0.0, 0.0);
		OutMaxValue = FVector(DistributionValue, 0.0, 0.0);
		return;
	}

	else if (Distribution->IsA<UDistributionVectorConstant>())
	{
		FVector DistributionValue = FVector(0.0f);
		GetVectorDistributionConstValues(static_cast<UDistributionVectorConstant*>(Distribution), DistributionValue);
		bOutSuccess = true;
		OutMinValue = DistributionValue;
		OutMaxValue = DistributionValue;
		return;
	}

	else if (Distribution->IsA<UDistributionFloatConstantCurve>())
	{
		UDistributionFloatConstantCurve* FloatCurveDistribution = static_cast<UDistributionFloatConstantCurve*>(Distribution);
		if (FloatCurveDistribution->ConstantCurve.Points.Num() == 0)
		{
			bOutSuccess = false;
			return;
		}

		float MinValue = FloatCurveDistribution->ConstantCurve.Points[0].OutVal;
		float MaxValue = FloatCurveDistribution->ConstantCurve.Points[0].OutVal;

		if (FloatCurveDistribution->ConstantCurve.Points.Num() > 1)
		{ 
			for (int i = 1; i < FloatCurveDistribution->ConstantCurve.Points.Num(); ++i)
			{
				const float& OutVal = FloatCurveDistribution->ConstantCurve.Points[i].OutVal;
				MinValue = OutVal < MinValue ? OutVal : MinValue;
				MaxValue = OutVal > MaxValue ? OutVal : MaxValue;
			}
		}

		bOutSuccess = true;
		OutMinValue = FVector(MinValue, 0.0, 0.0);
		OutMaxValue = FVector(MaxValue, 0.0, 0.0);
		return;
	}

	else if (Distribution->IsA<UDistributionVectorConstantCurve>())
	{
		UDistributionVectorConstantCurve* VectorCurveDistribution = static_cast<UDistributionVectorConstantCurve*>(Distribution);
		if (VectorCurveDistribution->ConstantCurve.Points.Num() == 0)
		{
			bOutSuccess = false;
			return;
		}

		OutMinValue = VectorCurveDistribution->ConstantCurve.Points[0].OutVal;
		OutMaxValue = VectorCurveDistribution->ConstantCurve.Points[0].OutVal;

		if (VectorCurveDistribution->ConstantCurve.Points.Num() > 1)
		{
			for (int i = 1; i < VectorCurveDistribution->ConstantCurve.Points.Num(); ++i)
			{
				const FVector& OutVal = VectorCurveDistribution->ConstantCurve.Points[i].OutVal;
				OutMinValue = OutVal.ComponentMin(OutMinValue);
				OutMaxValue = OutVal.ComponentMax(OutMaxValue);
			}
		}

		bOutSuccess = true;
		return;
	}

	else if (Distribution->IsA<UDistributionFloatUniform>())
	{
		float DistributionValueMin = 0.0f;
		float DistributionValueMax = 0.0f;
		GetFloatDistributionUniformValues(static_cast<UDistributionFloatUniform*>(Distribution), DistributionValueMin, DistributionValueMax);
		bOutSuccess = true;
		OutMinValue = FVector(DistributionValueMin, 0.0, 0.0);
		OutMaxValue = FVector(DistributionValueMax, 0.0, 0.0);
		return;
	}

	else if (Distribution->IsA<UDistributionVectorUniform>())
	{
		GetVectorDistributionUniformValues(static_cast<UDistributionVectorUniform*>(Distribution), OutMinValue, OutMaxValue);
		bOutSuccess = true;
		return;
	}

	else if (Distribution->IsA<UDistributionFloatUniformCurve>())
	{
		UDistributionFloatUniformCurve* FloatCurveDistribution = static_cast<UDistributionFloatUniformCurve*>(Distribution);
		if (FloatCurveDistribution->ConstantCurve.Points.Num() == 0)
		{
			bOutSuccess = false;
			return;
		}

		float MinValue = FloatCurveDistribution->ConstantCurve.Points[0].OutVal.X;
		float MaxValue = FloatCurveDistribution->ConstantCurve.Points[0].OutVal.Y;

		if (FloatCurveDistribution->ConstantCurve.Points.Num() > 1)
		{
			for (int i = 1; i < FloatCurveDistribution->ConstantCurve.Points.Num(); ++i)
			{
				const FVector2D& OutVal = FloatCurveDistribution->ConstantCurve.Points[i].OutVal;
				MinValue = OutVal.X < MinValue ? OutVal.X : MinValue;
				MaxValue = OutVal.Y > MaxValue ? OutVal.Y : MaxValue;
			}
		}

		bOutSuccess = true;
		OutMinValue = FVector(MinValue, 0.0, 0.0);
		OutMaxValue = FVector(MaxValue, 0.0, 0.0);
		return;
	}

	else if (Distribution->IsA<UDistributionVectorUniformCurve>())
	{
		UDistributionVectorUniformCurve* VectorCurveDistribution = static_cast<UDistributionVectorUniformCurve*>(Distribution);
		if (VectorCurveDistribution->ConstantCurve.Points.Num() == 0)
		{
			bOutSuccess = false;
			return;
		}
			
		OutMinValue = VectorCurveDistribution->ConstantCurve.Points[0].OutVal.v1;
		OutMaxValue = VectorCurveDistribution->ConstantCurve.Points[0].OutVal.v2;

		if (VectorCurveDistribution->ConstantCurve.Points.Num() > 1)
		{
			for (int i = 1; i < VectorCurveDistribution->ConstantCurve.Points.Num(); ++i)
			{
				const FTwoVectors& OutVal = VectorCurveDistribution->ConstantCurve.Points[i].OutVal;
				OutMinValue = OutVal.v1.ComponentMin(OutMinValue);
				OutMaxValue = OutVal.v2.ComponentMax(OutMaxValue);
			}
		}

		bOutSuccess = true;
		return;
	}

	else if (Distribution->IsA<UDistributionFloatParameterBase>())
	{
		bOutSuccess = false;
		return;
	}

	else if (Distribution->IsA<UDistributionVectorParameterBase>())
	{
		bOutSuccess = false;
		return;
	}

	bOutSuccess = false;
}

TArray<TEnumAsByte<EDistributionVectorLockFlags>> UFXConverterUtilitiesLibrary::GetDistributionLockedAxes(UDistribution* Distribution)
{
	TArray<TEnumAsByte<EDistributionVectorLockFlags>> OutLockAxesFlags;
	if (UDistributionVectorConstant* DistributionVectorConstant = Cast<UDistributionVectorConstant>(Distribution))
	{
		OutLockAxesFlags.Add(DistributionVectorConstant->LockedAxes);
	}
	else if (UDistributionVectorUniform* DistributionVectorUniform = Cast<UDistributionVectorUniform>(Distribution))
	{
		OutLockAxesFlags.Add(DistributionVectorUniform->LockedAxes);
	}
	else if (UDistributionVectorConstantCurve* DistributionVectorConstantCurve = Cast<UDistributionVectorConstantCurve>(Distribution))
	{
		OutLockAxesFlags.Add(DistributionVectorConstantCurve->LockedAxes);
	}
	else if(UDistributionVectorUniformCurve* DistributionVectorUniformCurve = Cast<UDistributionVectorUniformCurve>(Distribution))
	{
		for (TEnumAsByte<EDistributionVectorLockFlags>& LockedAxis : DistributionVectorUniformCurve->LockedAxes)
		{
			OutLockAxesFlags.Add(LockedAxis);
		}
	}
	else
	{
		OutLockAxesFlags.Add(EDistributionVectorLockFlags::EDVLF_None);
	}
	return OutLockAxesFlags;
}

void UFXConverterUtilitiesLibrary::GetDistributionType(
	UDistribution* Distribution
	, EDistributionType& OutDistributionType
	, EDistributionValueType& OutCascadeDistributionValueType)
{
	OutDistributionType = EDistributionType::NONE;
	OutCascadeDistributionValueType = EDistributionValueType::NONE;
	if (Distribution == nullptr)
	{
		return;
	}
	else if (Distribution->IsA<UDistributionFloatConstant>())
	{
		OutDistributionType = EDistributionType::Const;
		OutCascadeDistributionValueType = EDistributionValueType::Float;
		return;
	}
	else if (Distribution->IsA<UDistributionVectorConstant>())
	{
		OutDistributionType = EDistributionType::Const;
		OutCascadeDistributionValueType = EDistributionValueType::Vector;
		return;
	}
	else if (Distribution->IsA<UDistributionFloatConstantCurve>())
	{
		OutDistributionType = EDistributionType::ConstCurve;
		OutCascadeDistributionValueType = EDistributionValueType::Float;
		return;
	}
	else if (Distribution->IsA<UDistributionVectorConstantCurve>())
	{
		OutDistributionType = EDistributionType::ConstCurve;
		OutCascadeDistributionValueType = EDistributionValueType::Vector;
		return;
	}
	else if (Distribution->IsA<UDistributionFloatUniform>())
	{
		OutDistributionType = EDistributionType::Uniform;
		OutCascadeDistributionValueType = EDistributionValueType::Float;
		return;
	}
	else if (Distribution->IsA<UDistributionVectorUniform>())
	{
		OutDistributionType = EDistributionType::Uniform;
		OutCascadeDistributionValueType = EDistributionValueType::Vector;
		return;
	}
	else if (Distribution->IsA<UDistributionFloatUniformCurve>())
	{
		OutDistributionType = EDistributionType::UniformCurve;
		OutCascadeDistributionValueType = EDistributionValueType::Float;
		return;
	}
	else if (Distribution->IsA<UDistributionVectorUniformCurve>())
	{
		OutDistributionType = EDistributionType::UniformCurve;
		OutCascadeDistributionValueType = EDistributionValueType::Vector;
		return;
	}
	else if (Distribution->IsA<UDistributionFloatParameterBase>())
	{
		OutDistributionType = EDistributionType::Parameter;
		OutCascadeDistributionValueType = EDistributionValueType::Float;
		return;
	}
	else if (Distribution->IsA<UDistributionVectorParameterBase>())
	{
		OutDistributionType = EDistributionType::Parameter;
		OutCascadeDistributionValueType = EDistributionValueType::Vector;
		return;
	}
}

void UFXConverterUtilitiesLibrary::GetFloatDistributionConstValues(UDistributionFloatConstant* Distribution, float& OutConstFloat)
{
	OutConstFloat = Distribution ? Distribution->GetValue() : 0.0f;
}

void UFXConverterUtilitiesLibrary::GetVectorDistributionConstValues(UDistributionVectorConstant* Distribution, FVector& OutConstVector)
{
	OutConstVector = Distribution ? Distribution->GetValue() : FVector::ZeroVector;
}

void UFXConverterUtilitiesLibrary::GetFloatDistributionUniformValues(UDistributionFloatUniform* Distribution, float& OutMin, float& OutMax)
{
	OutMin = Distribution ? Distribution->Min : 0.0f;
	OutMax = Distribution ? Distribution->Max : 0.0f;
}

void UFXConverterUtilitiesLibrary::GetVectorDistributionUniformValues(UDistributionVectorUniform* Distribution, FVector& OutMin, FVector& OutMax)
{
	OutMin = Distribution ? Distribution->Min : FVector::ZeroVector;
	OutMax = Distribution ? Distribution->Max : FVector::ZeroVector;
}

void UFXConverterUtilitiesLibrary::GetFloatDistributionConstCurveValues(UDistributionFloatConstantCurve* Distribution, FInterpCurveFloat& OutInterpCurveFloat)
{
	if (Distribution)
	{
		OutInterpCurveFloat = Distribution->ConstantCurve;
	}
}

void UFXConverterUtilitiesLibrary::GetVectorDistributionConstCurveValues(UDistributionVectorConstantCurve* Distribution, FInterpCurveVector& OutInterpCurveVector)
{
	if (Distribution)
	{
		OutInterpCurveVector = Distribution->ConstantCurve;
	}
}

void UFXConverterUtilitiesLibrary::GetFloatDistributionUniformCurveValues(UDistributionFloatUniformCurve* Distribution, FInterpCurveVector2D& OutInterpCurveVector2D)
{
	if (Distribution)
	{
		OutInterpCurveVector2D = Distribution->ConstantCurve;
	}
}

void UFXConverterUtilitiesLibrary::GetVectorDistributionUniformCurveValues(UDistributionVectorUniformCurve* Distribution, FInterpCurveTwoVectors& OutInterpCurveTwoVectors)
{
	if (Distribution)
	{
		OutInterpCurveTwoVectors = Distribution->ConstantCurve;
	}
}

void UFXConverterUtilitiesLibrary::GetFloatDistributionParameterValues(UDistributionFloatParameterBase* Distribution, FName& OutParameterName, float& OutMinInput, float& OutMaxInput, float& OutMinOutput, float& OutMaxOutput)
{
	if (Distribution)
	{
		OutParameterName = Distribution->ParameterName;
		OutMinInput = Distribution->MinInput;
		OutMaxInput = Distribution->MaxInput;
		OutMinOutput = Distribution->MinOutput;
		OutMaxOutput = Distribution->MaxOutput;
	}
	else
	{
		OutParameterName = NAME_None;
		OutMinInput = 0.0f;
		OutMaxInput = 0.0f;
		OutMinOutput = 0.0f;
		OutMaxOutput = 0.0f;
	}
}

void UFXConverterUtilitiesLibrary::GetVectorDistributionParameterValues(UDistributionVectorParameterBase* Distribution, FName& OutParameterName, FVector& OutMinInput, FVector& OutMaxInput, FVector& OutMinOutput, FVector& OutMaxOutput)
{
	if (Distribution)
	{
		OutParameterName = Distribution->ParameterName;
		OutMinInput = Distribution->MinInput;
		OutMaxInput = Distribution->MaxInput;
		OutMinOutput = Distribution->MinOutput;
		OutMaxOutput = Distribution->MaxOutput;
	}
	else
	{
		OutParameterName = NAME_None;
		OutMinInput = FVector::ZeroVector;
		OutMaxInput = FVector::ZeroVector;
		OutMinOutput = FVector::ZeroVector;
		OutMaxOutput = FVector::ZeroVector;
	}
}

TArray<FRichCurveKeyBP> UFXConverterUtilitiesLibrary::KeysFromInterpCurveFloat(FInterpCurveFloat Curve)
{
	TArray<FRichCurveKeyBP> Keys;
	for (const FInterpCurvePoint<float>& Point : Curve.Points)
	{
		Keys.Emplace(FRichCurveKey(Point));
	}
	return Keys;
}

TArray<FRichCurveKeyBP> UFXConverterUtilitiesLibrary::KeysFromInterpCurveVector(FInterpCurveVector Curve, int32 ComponentIdx)
{
	TArray<FRichCurveKeyBP> Keys;
	for (const FInterpCurvePoint<FVector>& Point : Curve.Points)
	{
		Keys.Emplace(FRichCurveKey(Point, ComponentIdx));
	}
	return Keys;
}

TArray<FRichCurveKeyBP> UFXConverterUtilitiesLibrary::KeysFromInterpCurveVector2D(FInterpCurveVector2D Curve, int32 ComponentIdx)
{
	TArray<FRichCurveKeyBP> Keys;
	for (const FInterpCurvePoint<FVector2D>& Point : Curve.Points)
	{
		Keys.Emplace(FRichCurveKey(Point, ComponentIdx));
	}
	return Keys;
}

TArray<FRichCurveKeyBP> UFXConverterUtilitiesLibrary::KeysFromInterpCurveTwoVectors(FInterpCurveTwoVectors Curve, int32 ComponentIdx)
{
	TArray<FRichCurveKeyBP> Keys;
	for (const FInterpCurvePoint<FTwoVectors>& Point : Curve.Points)
	{
		Keys.Emplace(FRichCurveKey(Point, ComponentIdx));
	}
	return Keys;
}

void UNiagaraSystemConversionContext::Cleanup()
{
	for (auto EmitterConversionContextIt(EmitterNameToConversionContextMap.CreateIterator()); EmitterConversionContextIt; ++EmitterConversionContextIt)
	{
		EmitterConversionContextIt.Value()->Cleanup();
	}

	System = nullptr;
	SystemViewModel.Reset();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	UNiagaraSystemConversionContext																			  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
UNiagaraEmitterConversionContext* UNiagaraSystemConversionContext::AddEmptyEmitter(FString NewEmitterNameString)
{
	return AddTemplateEmitter(NewEmitterNameString, TEXT("/Niagara/DefaultAssets/Templates/CascadeConversion/CompletelyEmpty"));
}

UNiagaraEmitterConversionContext* UNiagaraSystemConversionContext::AddTemplateEmitter(FString NewEmitterNameString, FString Template)
{
	FName NewEmitterName = FNiagaraEditorUtilities::GetUniqueObjectName(System, UNiagaraEmitter::StaticClass(), NewEmitterNameString);
	UNiagaraEmitter* NewEmitter = CastChecked<UNiagaraEmitter>(StaticLoadObject(UNiagaraEmitter::StaticClass(), nullptr, *Template));
	
	const TSharedPtr<FNiagaraEmitterHandleViewModel>& NewEmitterHandleViewModel = SystemViewModel->AddEmitter(*NewEmitter, NewEmitter->GetExposedVersion().VersionGuid);
	NewEmitterHandleViewModel->SetName(NewEmitterName);
	NewEmitterName = NewEmitterHandleViewModel->GetName();

	UNiagaraEmitterConversionContext* EmitterConversionContext = NewObject<UNiagaraEmitterConversionContext>();
	EmitterConversionContext->Init(NewEmitterHandleViewModel->GetEmitterHandle()->GetInstance(), NewEmitterHandleViewModel);
	EmitterNameToConversionContextMap.Add(NewEmitterName, EmitterConversionContext);
	return EmitterConversionContext;
}

void UNiagaraSystemConversionContext::SetWarmupTime(float InWarmupTime)
{
	this->System->SetWarmupTime(InWarmupTime);
}

void UNiagaraSystemConversionContext::SetWarmupTickDelta(float InWarmupTickDelta)
{
	this->System->SetWarmupTickDelta(InWarmupTickDelta);
}

void UNiagaraSystemConversionContext::Finalize()
{
	// Resolve events before stack entries, as events may add stack categories for the stack entries.
	for (auto EmitterConversionContextIt(EmitterNameToConversionContextMap.CreateIterator()); EmitterConversionContextIt; ++EmitterConversionContextIt)
	{
		EmitterConversionContextIt.Value()->InternalFinalizeEvents(this);
	}
	SystemViewModel->GetSystemStackViewModel()->GetRootEntry()->RefreshChildren();

	for (auto EmitterConversionContextIt(EmitterNameToConversionContextMap.CreateIterator()); EmitterConversionContextIt; ++EmitterConversionContextIt)
	{
		EmitterConversionContextIt.Value()->InternalFinalizeStackEntryAddActions();
	}

	System->RequestCompile(false);
}

UNiagaraEmitterConversionContext* UNiagaraSystemConversionContext::FindEmitterConversionContextByName(const FName& EmitterName)
{
	TObjectPtr<UNiagaraEmitterConversionContext>* FoundContext = EmitterNameToConversionContextMap.Find(EmitterName);
	return FoundContext ? FoundContext->Get() : nullptr;
}

void UNiagaraEmitterConversionContext::Cleanup()
{
	Emitter = FVersionedNiagaraEmitter();
	EmitterHandleViewModel.Reset();
	StackEntryAddActions.Empty();
	RendererNameToStagedRendererPropertiesMap.Empty();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	UNiagaraEmitterConversionContext																		  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
UNiagaraScriptConversionContext* UNiagaraEmitterConversionContext::FindOrAddModuleScript(FString ScriptNameString, FCreateScriptContextArgs CreateScriptContextArgs, EScriptExecutionCategory ModuleScriptExecutionCategory)
{
	return PrivateFindOrAddModuleScript(ScriptNameString, CreateScriptContextArgs, FStackEntryID(ModuleScriptExecutionCategory));
}

UNiagaraScriptConversionContext* UNiagaraEmitterConversionContext::FindOrAddModuleEventScript(FString ScriptNameString, FCreateScriptContextArgs CreateScriptContextArgs, FNiagaraEventHandlerAddAction EventHandlerAddAction)
{
	return PrivateFindOrAddModuleScript(ScriptNameString, CreateScriptContextArgs, FStackEntryID(EventHandlerAddAction));
}

UNiagaraScriptConversionContext* UNiagaraEmitterConversionContext::PrivateFindOrAddModuleScript(const FString& ScriptNameString, const FCreateScriptContextArgs& CreateScriptContextArgs, const FStackEntryID& StackEntryID)
{
	const FName ScriptName = FName(ScriptNameString);
	FStackEntryAddAction* StackEntryAddAction = StackEntryAddActions.FindByPredicate([&ScriptName](const FStackEntryAddAction& AddAction) {return AddAction.Mode == EStackEntryAddActionMode::Module && AddAction.ModuleName == ScriptName; });
	if (StackEntryAddAction != nullptr)
	{
		return StackEntryAddAction->ScriptConversionContext;
	}

	UNiagaraScriptConversionContext* ScriptContext = UFXConverterUtilitiesLibrary::CreateScriptContext(CreateScriptContextArgs);
	StackEntryAddActions.Emplace(ScriptContext, StackEntryID, ScriptName);
	return ScriptContext;
}

void UNiagaraEmitterConversionContext::RemoveModuleScriptsForAssets(TArray<FAssetData> ScriptsToRemove)
{
	TArray<FString> ScriptPathsToRemove;
	for (const FAssetData& AssetData : ScriptsToRemove)
	{
		ScriptPathsToRemove.Add(AssetData.PackagePath.ToString());
	}

	for (int i = StackEntryAddActions.Num(); i > 0; --i)
	{
		if (StackEntryAddActions[i].Mode == EStackEntryAddActionMode::Module)
		{
			const FString& ScriptPathNameString = StackEntryAddActions[i].ScriptConversionContext->GetScript()->GetPathName();
			if (ScriptPathsToRemove.Contains(ScriptPathNameString))
			{
				StackEntryAddActions.RemoveAt(i);
			}
		}
	}
}

UNiagaraScriptConversionContext* UNiagaraEmitterConversionContext::FindModuleScript(FString ScriptNameString)
{
	const FName ScriptName = FName(ScriptNameString);
	FStackEntryAddAction* StackEntryAddAction = StackEntryAddActions.FindByPredicate([&ScriptName](const FStackEntryAddAction& AddAction) {return AddAction.Mode == EStackEntryAddActionMode::Module && AddAction.ModuleName == ScriptName; });
	if (StackEntryAddAction != nullptr)
	{
		return StackEntryAddAction->ScriptConversionContext;
	}
	return nullptr;
}

void UNiagaraEmitterConversionContext::AddModuleScript(FString ScriptNameString, UNiagaraScriptConversionContext* ScriptConversionContext, EScriptExecutionCategory ModuleScriptExecutionCategory)
{
	StackEntryAddActions.Emplace(ScriptConversionContext, FStackEntryID(ModuleScriptExecutionCategory), FName(ScriptNameString));
}

void UNiagaraEmitterConversionContext::AddModuleEventScript(FString ScriptNameString, UNiagaraScriptConversionContext* ScriptConversionContext, FNiagaraEventHandlerAddAction EventHandlerAddAction)
{
	StackEntryAddActions.Emplace(ScriptConversionContext, FStackEntryID(EventHandlerAddAction), FName(ScriptNameString));
}

void UNiagaraEmitterConversionContext::SetParameterDirectly(FString ParameterNameString, UNiagaraScriptConversionContextInput* ParameterInput, EScriptExecutionCategory SetParameterExecutionCategory)
{
	const FName ParameterName = FName(ParameterNameString);
	const FNiagaraVariable TargetVariable = FNiagaraVariable(ParameterInput->TypeDefinition, ParameterName);
	const TArray<FNiagaraVariable> InVariables = {TargetVariable};
	const TArray<FString> InVariableDefaults = {FString()};
	UNiagaraClipboardFunction* Assignment = UNiagaraClipboardFunction::CreateAssignmentFunction(this, "SetParameter", InVariables, InVariableDefaults);
	ParameterInput->ClipboardFunctionInput->InputName = ParameterName;
	Assignment->Inputs.Add(ParameterInput->ClipboardFunctionInput);
	StackEntryAddActions.Emplace(Assignment, FStackEntryID(SetParameterExecutionCategory));
}

void UNiagaraEmitterConversionContext::AddRenderer(FString RendererNameString, UNiagaraRendererProperties* NewRendererProperties)
{
	RendererNameToStagedRendererPropertiesMap.Add(RendererNameString, NewRendererProperties);
}

UNiagaraRendererProperties* UNiagaraEmitterConversionContext::FindRenderer(FString RendererNameString)
{
	TObjectPtr<UNiagaraRendererProperties>* PropsPtr = RendererNameToStagedRendererPropertiesMap.Find(RendererNameString);
	return PropsPtr ? PropsPtr->Get() : nullptr;
}

TArray<UNiagaraRendererProperties*> UNiagaraEmitterConversionContext::GetAllRenderers()
{
	TArray<TObjectPtr<UNiagaraRendererProperties>> OutRendererProperties;
	RendererNameToStagedRendererPropertiesMap.GenerateValueArray(OutRendererProperties);
	return OutRendererProperties;
}

void UNiagaraEmitterConversionContext::Log(FString Message, ENiagaraMessageSeverity Severity, bool bIsVerbose /*= false*/)
{
	EmitterMessages.Add(FGenericConverterMessage(Message, Severity, bIsVerbose));
}

void UNiagaraEmitterConversionContext::Finalize()
{
	// If finalizing directly from an Emitter, pass a nullptr for the OwningSystemConversionContext.
	InternalFinalizeEvents(nullptr);
	InternalFinalizeStackEntryAddActions();
}

void UNiagaraEmitterConversionContext::InternalFinalizeEvents(UNiagaraSystemConversionContext* OwningSystemConversionContext)
{
	const TSharedRef<FNiagaraEmitterViewModel>& EmitterViewModel = EmitterHandleViewModel->GetEmitterViewModel();
	bool bFinalizingFromSystemConversionContext = OwningSystemConversionContext != nullptr;
	
	// Add event handlers.
	for (FNiagaraEventHandlerAddAction& EventHandlerAddAction : EventHandlerAddActions)
	{
		// If adding an event generator is specified and finalizing from an owning system conversion context, find the Emitter with the source name and add a paired event generator.
		if (EventHandlerAddAction.Mode == ENiagaraEventHandlerAddMode::AddEventAndEventGenerator && bFinalizingFromSystemConversionContext)
		{
			const FName& SourceEmitterName = EventHandlerAddAction.AddEventGeneratorOptions.SourceEmitterName;
			if (UNiagaraEmitterConversionContext* SourceEmitterConversionContext = OwningSystemConversionContext->FindEmitterConversionContextByName(SourceEmitterName))
			{
				// Enable persistent IDs for the event generator emitter as this is a requisite for events.
				SourceEmitterConversionContext->Emitter.GetEmitterData()->bRequiresPersistentIDs = true;

				// Add the event generator.
				SourceEmitterConversionContext->FindOrAddModuleScript("EventGenerator", EventHandlerAddAction.AddEventGeneratorOptions.EventGeneratorScriptAssetData, EScriptExecutionCategory::ParticleUpdate);
				EventHandlerAddAction.SourceEmitterID = SourceEmitterConversionContext->GetEmitterHandleId();
			}
			else
			{
				Log("Converter failed to find Emitter named: \"" + SourceEmitterName.ToString() + "\" to add event generator to.", ENiagaraMessageSeverity::Error, false);
			}
		}

		// Add the event handler category to the stack.
		FNiagaraEventScriptProperties EventScriptProps = EventHandlerAddAction.GetEventScriptProperties();
		EmitterViewModel->AddEventHandler(EventScriptProps, true);

		// Force refresh the stack so that the stack item group for events is generated and may be referenced when pasting modules during InternalFinalizeStackEntryAddActions().
		EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry()->RefreshChildren();
	}
}

void UNiagaraEmitterConversionContext::InternalFinalizeStackEntryAddActions()
{
	// Get viewmodels and collect all stack item groups for adding entries to.
	TSharedRef<FNiagaraSystemViewModel> OwningSystemViewModel = EmitterHandleViewModel->GetOwningSystemViewModel();
	TArray<UNiagaraStackItemGroup*> StackItemGroups;
	EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry()->GetUnfilteredChildrenOfType<UNiagaraStackItemGroup>(StackItemGroups);
	EmitterHandleViewModel->SetIsEnabled(bEnabled);

	// Helper lambda to get the UNiagaraStackItemGroup for any FStackEntryID.
	auto GetStackItemGroupForStackEntryID = [&StackItemGroups, this](const FStackEntryID& StackEntryID)->UNiagaraStackItemGroup* const* {
		const EScriptExecutionCategory& ExecutionCategory = StackEntryID.ScriptExecutionCategory;

		if (ExecutionCategory == EScriptExecutionCategory::EmitterSpawn ||
			ExecutionCategory == EScriptExecutionCategory::EmitterUpdate ||
			ExecutionCategory == EScriptExecutionCategory::ParticleSpawn ||
			ExecutionCategory == EScriptExecutionCategory::ParticleUpdate)
		{
			FName ExecutionCategoryName;
			FName ExecutionSubcategoryName;

			switch (ExecutionCategory) {
			case EScriptExecutionCategory::EmitterSpawn:
				ExecutionCategoryName = UNiagaraStackEntry::FExecutionCategoryNames::Emitter;
				ExecutionSubcategoryName = UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn;
				break;
			case EScriptExecutionCategory::EmitterUpdate:
				ExecutionCategoryName = UNiagaraStackEntry::FExecutionCategoryNames::Emitter;
				ExecutionSubcategoryName = UNiagaraStackEntry::FExecutionSubcategoryNames::Update;
				break;
			case EScriptExecutionCategory::ParticleSpawn:
				ExecutionCategoryName = UNiagaraStackEntry::FExecutionCategoryNames::Particle;
				ExecutionSubcategoryName = UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn;
				break;
			case EScriptExecutionCategory::ParticleUpdate:
				ExecutionCategoryName = UNiagaraStackEntry::FExecutionCategoryNames::Particle;
				ExecutionSubcategoryName = UNiagaraStackEntry::FExecutionSubcategoryNames::Update;
				break;
			default:
				ensureMsgf(false, TEXT("Encountered unexpected EScriptExecutionCategory!"));
			};

			auto FilterStackEntryForItemGroup = [ExecutionCategoryName, ExecutionSubcategoryName](UNiagaraStackItemGroup* ItemGroup)-> bool {
				return ItemGroup->GetExecutionCategoryName() == ExecutionCategoryName && ItemGroup->GetExecutionSubcategoryName() == ExecutionSubcategoryName;
			};

			return StackItemGroups.FindByPredicate(FilterStackEntryForItemGroup);
		}
		else if (ExecutionCategory == EScriptExecutionCategory::ParticleEvent)
		{
			auto GetEventSourceEmitterId = [](const FName EventName, FVersionedNiagaraEmitter InEmitter)->const FGuid {
				const FNiagaraEventScriptProperties* FoundEventProps = InEmitter.GetEmitterData()->GetEventHandlers().FindByPredicate(
					[EventName](const FNiagaraEventScriptProperties& EventProps) {return EventProps.SourceEventName == EventName; }
				);
				if (FoundEventProps != nullptr)
				{
					return FoundEventProps->SourceEmitterID;
				}
				return FGuid();
			};

			FVersionedNiagaraEmitter EmitterInstance = EmitterHandleViewModel->GetEmitterHandle()->GetInstance();
			const FGuid EventSourceEmitterId = GetEventSourceEmitterId(StackEntryID.EventName, EmitterInstance);

			auto FilterStackEntryForEventSourceEmitterId = [EventSourceEmitterId](UNiagaraStackItemGroup* ItemGroup)->bool {
				const UNiagaraStackEventScriptItemGroup* EventItemGroup = Cast<const UNiagaraStackEventScriptItemGroup>(ItemGroup);
				if (EventItemGroup != nullptr)
				{
					return EventItemGroup->GetEventSourceEmitterId() == EventSourceEmitterId;
				}
				return false;
			};

			return StackItemGroups.FindByPredicate(FilterStackEntryForEventSourceEmitterId);
		}

		ensureMsgf(false, TEXT("Encountered unknown EScriptExecutionCategory when choosing script to add module to emitter!"));
		return nullptr;
	};

	// Filter the StackEntryAddActions into modes.
	TArray<FStackEntryAddAction> StackSetParameterActions = StackEntryAddActions.FilterByPredicate([](const FStackEntryAddAction& AddAction) {return AddAction.Mode == EStackEntryAddActionMode::SetParameter; });
	TArray<FStackEntryAddAction> StackAddModuleActions = StackEntryAddActions.FilterByPredicate([](const FStackEntryAddAction& AddAction) {return AddAction.Mode == EStackEntryAddActionMode::Module; });

	auto ApplySetParameterAction = [&StackItemGroups, &GetStackItemGroupForStackEntryID](const FStackEntryAddAction& SetParameterAction) {
		if (SetParameterAction.ClipboardFunction == nullptr)
		{
			ensureMsgf(false, TEXT("FStackEntryAddAction with Mode SetParameter did not have valid ClipboardFunction ptr!"));
			return;
		}

		UNiagaraStackItemGroup* const* TargetStackItemGroupPtr = GetStackItemGroupForStackEntryID(SetParameterAction.StackEntryID);
		if (TargetStackItemGroupPtr == nullptr)
		{
			ensureMsgf(false, TEXT("Failed to get StackItemGroup for StackEntryAddAction!"));
			return;
		}

		UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
		ClipboardContent->Functions.Add(SetParameterAction.ClipboardFunction);

		FText PasteWarning = FText();
		UNiagaraStackItemGroup* TargetStackItemGroup = *TargetStackItemGroupPtr;
		TargetStackItemGroup->Paste(ClipboardContent, PasteWarning);

		if (PasteWarning.IsEmpty() == false)
		{
			UE_LOGF(LogFXConverter, Error, "%ls", *PasteWarning.ToString());
			return;
		}
	};

	auto ApplyAddModuleAction = [&OwningSystemViewModel, &StackItemGroups, &GetStackItemGroupForStackEntryID, this](FStackEntryAddAction& AddModuleAction) {
		UNiagaraScriptConversionContext* ScriptConversionContext = AddModuleAction.ScriptConversionContext;
		if (ScriptConversionContext == nullptr)
		{
			ensureMsgf(false, TEXT("FStackEntryAddAction with Mode Module did not have valid ScriptConversionContext ptr!"));
			return;
		}
		UNiagaraStackItemGroup* const* TargetStackItemGroupPtr = GetStackItemGroupForStackEntryID(AddModuleAction.StackEntryID);

		if (TargetStackItemGroupPtr == nullptr)
		{
			ensureMsgf(false, TEXT("Failed to get StackItemGroup for StackEntryAddAction!"));
			return;
		}

		UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
		ClipboardContent->bFixupPasteIndexForScriptDependenciesInStack = true;
		UNiagaraScript* NiagaraScript = ScriptConversionContext->GetScript();
		const FGuid& ScriptVersionGuid = ScriptConversionContext->GetScriptVersionGuid();

		UNiagaraClipboardFunction* ClipboardFunction = UNiagaraClipboardFunction::CreateScriptFunction(ClipboardContent, "Function", NiagaraScript, ScriptVersionGuid);
		ClipboardFunction->Inputs = ScriptConversionContext->GetClipboardFunctionInputs();
		ClipboardContent->Functions.Add(ClipboardFunction);

		ClipboardFunction->OnPastedFunctionCallNodeDelegate.BindDynamic(this, &UNiagaraEmitterConversionContext::SetPastedFunctionCallNode);

		// Commit the clipboard content to the target stack entry
		FText PasteWarning = FText();
		UNiagaraStackItemGroup* TargetStackItemGroup = *TargetStackItemGroupPtr;
		TargetStackItemGroup->Paste(ClipboardContent, PasteWarning);
		ClipboardFunction->OnPastedFunctionCallNodeDelegate.Unbind();

		if (PasteWarning.IsEmpty() == false)
		{
			UE_LOGF(LogFXConverter, Error, "%ls", *PasteWarning.ToString());
			return;
		}

		if (PastedFunctionCallNode != nullptr)
		{
			// Set the module enabled state
			if (ScriptConversionContext->GetModuleEnabled() == false)
			{
				FNiagaraStackGraphUtilities::SetModuleIsEnabled(*PastedFunctionCallNode, false);
			}

			// Push the per module messages
			for (const FGenericConverterMessage& Message : ScriptConversionContext->GetStackMessages())
			{
				UNiagaraMessageDataText* NewMessageDataText = NewObject<UNiagaraMessageDataText>(PastedFunctionCallNode);
				const FName TopicName = Message.bIsVerbose ? FNiagaraConverterMessageTopics::VerboseConversionEventTopicName : FNiagaraConverterMessageTopics::ConversionEventTopicName;
				NewMessageDataText->Init(FText::FromString(Message.Message), Message.MessageSeverity, TopicName);
				OwningSystemViewModel->AddStackMessage(NewMessageDataText, PastedFunctionCallNode);
			}
			PastedFunctionCallNode = nullptr;
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Did not receive a function call from the niagara clipboard module paste event!"));
		}
	};

	// Add the set parameter stack entries.
	for (const FStackEntryAddAction& SetParameterAction : StackSetParameterActions)
	{
		ApplySetParameterAction(SetParameterAction);
	}

	// Add the module script stack entries.
	for (FStackEntryAddAction& AddModuleAction : StackAddModuleActions)
	{
		ApplyAddModuleAction(AddModuleAction);
	}

	// Find the renderer stack item group.
	UNiagaraStackItemGroup* const* RendererStackItemGroupPtr = StackItemGroups.FindByPredicate([](const UNiagaraStackItemGroup* EmitterItemGroup) {
		return EmitterItemGroup->GetExecutionCategoryName() == UNiagaraStackEntry::FExecutionCategoryNames::Render
			&& EmitterItemGroup->GetExecutionSubcategoryName() == UNiagaraStackEntry::FExecutionSubcategoryNames::Render; });

	if (ensureMsgf(RendererStackItemGroupPtr != nullptr, TEXT("Failed to find renderer stack items group for Emitter!")))
	{
		// Add the staged renderer properties.
		for (auto It = RendererNameToStagedRendererPropertiesMap.CreateIterator(); It; ++It)
		{
			UNiagaraRendererProperties* NewRendererProperties = It.Value();

			UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
			ClipboardContent->Renderers.Add(UNiagaraClipboardRenderer::CreateRenderer(ClipboardContent, NewRendererProperties));

			FText PasteWarning = FText();
			UNiagaraStackItemGroup* RendererStackItemGroup = *RendererStackItemGroupPtr;
			RendererStackItemGroup->Paste(ClipboardContent, PasteWarning);
			if (PasteWarning.IsEmpty() == false)
			{
				UE_LOGF(LogFXConverter, Warning, "%ls", *PasteWarning.ToString());
			}
		}
	}

	// Push the emitter messages.
	for (FGenericConverterMessage& Message : EmitterMessages)
	{
		UNiagaraMessageDataText* NewMessageDataText = NewObject<UNiagaraMessageDataText>(Emitter.Emitter);
		const FName TopicName = Message.bIsVerbose ? FNiagaraConverterMessageTopics::VerboseConversionEventTopicName : FNiagaraConverterMessageTopics::ConversionEventTopicName;
		NewMessageDataText->Init(FText::FromString(Message.Message), Message.MessageSeverity, TopicName);
		EmitterHandleViewModel->AddMessage(NewMessageDataText);
	}
}

FGuid UNiagaraEmitterConversionContext::GetEmitterHandleId() const
{
	return EmitterHandleViewModel->GetId();
}

void UNiagaraEmitterConversionContext::SetDetailBitMask(int32 BitMask)
{
	FVersionedNiagaraEmitterData* VersionedEmitterData = Emitter.GetEmitterData();
	if ( !VersionedEmitterData )
	{
		return;
	}

	if (BitMask == PDM_DefaultValue)
	{
		return;
	}

	const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();
	const static UEnum* CascadeDetailModeEnum = StaticEnum<EParticleDetailMode>();

	for (int32 i=0; i < int32(EParticleDetailMode::PDM_MAX); ++i)
	{
		if ((BitMask & (1 << i)) != 0)
		{
			continue;
		}

		const FText CascadeDetailText = CascadeDetailModeEnum->GetDisplayNameTextByValue(i);
		const int32 NiagaraBitIndex = NiagaraSettings->QualityLevels.IndexOfByPredicate([&CascadeDetailText](const FText& Value) { return Value.EqualTo(CascadeDetailText); });
		if (NiagaraBitIndex == INDEX_NONE)
		{
			UE_LOGF(LogFXConverter, Error, "Failed to find Niagara quality level for \"%ls\", this will be ignored!", *CascadeDetailText.ToString());
			continue;
		}

		VersionedEmitterData->Platforms.QualityLevelMask &= ~(1 << i);
	}
}

void UNiagaraEmitterConversionContext::AddEventHandler(FNiagaraEventHandlerAddAction EventHandlerAddAction)
{
	EventHandlerAddActions.Add(EventHandlerAddAction);
}

void UNiagaraEmitterConversionContext::SetRendererBinding(UNiagaraRendererProperties* InRendererProperties, FName BindingName, FName VariableToBindName, ENiagaraRendererSourceDataMode SourceDataMode)
{
	if (UNiagaraComponentRendererProperties* ComponentRendererProperties = Cast<UNiagaraComponentRendererProperties>(InRendererProperties))
	{
		for (FNiagaraComponentPropertyBinding& ComponentPropertyBinding : ComponentRendererProperties->PropertyBindings)
		{
			if (ComponentPropertyBinding.PropertyName == BindingName)
			{
				ComponentPropertyBinding.AttributeBinding.SetValue(VariableToBindName, Emitter.ToBase(), SourceDataMode);
				return;
			}
		}
		UE_LOGF(LogFXConverter, Error, "Tried to set component renderer binding \"%ls\" but it was not supported for component!", *BindingName.ToString());
		return;
	}

	FProperty* ObjectProp = InRendererProperties->GetClass()->FindPropertyByName(BindingName);
	if (!ObjectProp)
	{
		UE_LOGF(LogFXConverter, Error, "Tried to set renderer binding \"%ls\" but the property could not be found!", *BindingName.ToString());
		return;
	}

	FStructProperty* ObjectStructProp = CastField<FStructProperty>(ObjectProp);
	if (!ObjectStructProp)
	{
		UE_LOGF(LogFXConverter, Error, "Tried to set renderer binding \"%ls\" but the property could be cast to a struct property!", *BindingName.ToString());
		return;
	}

	if (!ObjectStructProp->Struct)
	{
		UE_LOGF(LogFXConverter, Error, "Tried to set renderer binding \"%ls\" but the struct property did not have a valid UStruct!", *BindingName.ToString());
		return;
	}

	if (!ObjectStructProp->Struct->IsChildOf(FNiagaraVariableAttributeBinding::StaticStruct()))
	{
		UE_LOGF(LogFXConverter, Error, "Tried to set renderer binding \"%ls\" but the ustruct of the property was not a child of FNiagaraVariableAttributeBinding static struct!", *BindingName.ToString());
		return;
	}

	FNiagaraVariableAttributeBinding* AttributeBinding = ObjectStructProp->ContainerPtrToValuePtr<FNiagaraVariableAttributeBinding>(InRendererProperties);
	if(!AttributeBinding)
	{
		UE_LOGF(LogFXConverter, Error, "Tried to set renderer binding \"%ls\" but failed to get the container to value ptr of the FPropertyStruct!", *BindingName.ToString());
		return;
	}

	AttributeBinding->SetValue(VariableToBindName, Emitter.ToBase(), SourceDataMode);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	UNiagaraScriptConversionContext																			  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void UNiagaraScriptConversionContext::Init(const FAssetData& InNiagaraScriptAssetData, TOptional<FNiagaraScriptVersion> InNiagaraScriptVersion /*= TOptional<FNiagaraScriptVersion>()*/)
{
	Script = static_cast<UNiagaraScript*>(InNiagaraScriptAssetData.GetAsset());
	if (Script == nullptr)
	{
		Log("Failed to create script! AssetData path was invalid!: " + InNiagaraScriptAssetData.PackagePath.ToString(), ENiagaraMessageSeverity::Error);
		return;
	}
	
	if (InNiagaraScriptVersion.IsSet())
	{
		bool bFoundVersion = false;
		const int32 InMajorVersion = InNiagaraScriptVersion->MajorVersion;
		const int32 InMinorVersion = InNiagaraScriptVersion->MinorVersion;
		for (const FNiagaraAssetVersion& Version : Script->GetAllAvailableVersions())
		{
			if (Version.MajorVersion == InMajorVersion && Version.MinorVersion == InMinorVersion)
			{
				ScriptVersionGuid = Version.VersionGuid;
				bFoundVersion = true;
				break;
			}
		}
		
		if (bFoundVersion == false)
		{
			Log(FString::Printf(TEXT("Failed to get script version! Supplied major version (%d) and minor version (%d) did not map to an available version!"), InMajorVersion, InMinorVersion), ENiagaraMessageSeverity::Error);
			return;
		}
	}

	// Gather the inputs to this script and add them to the lookup table for validating UNiagaraScriptConversionContextInputs that are set.
	TArray<UNiagaraNodeInput*> InputNodes;

	UNiagaraGraph* ScriptSourceGraph = nullptr;
	if (ScriptVersionGuid.IsValid())
	{
		ScriptSourceGraph = static_cast<UNiagaraScriptSource*>(Script->GetSource(ScriptVersionGuid))->NodeGraph;
	}
	else
	{
		ScriptSourceGraph = static_cast<UNiagaraScriptSource*>(Script->GetLatestSource())->NodeGraph;
	}

	const TMap<FNiagaraVariable, FInputPinsAndOutputPins> VarToPinsMap = ScriptSourceGraph->CollectVarsToInOutPinsMap();
	for (auto It = VarToPinsMap.CreateConstIterator(); It; ++It)
	{
		if (It->Value.OutputPins.Num() > 0)
		{
			const FNiagaraVariable& Var = It->Key;
			FName NamespacelessName;

			TArray<FName> NameSpaces = FNiagaraEditorUtilities::DecomposeVariableNamespace(Var.GetName(), NamespacelessName);

			if(NameSpaces.Contains("Module"))
			{
				InputNameToTypeDefMap.Add(NamespacelessName.ToString(), Var.GetType());
			}
		}
	}
	
	const TArray<FNiagaraVariable> StaticSwitchVars = ScriptSourceGraph->FindStaticSwitchInputs();
	for (const FNiagaraVariable& Var : StaticSwitchVars)
	{
		InputNameToTypeDefMap.Add(FNiagaraEditorUtilities::GetNamespacelessVariableNameString(Var.GetName()), Var.GetType());
	}
}

bool UNiagaraScriptConversionContext::SetParameter(FString ParameterName, UNiagaraScriptConversionContextInput* ParameterInput, bool bInHasEditCondition /*= false*/, bool bInEditConditionValue /* = false*/)
{
	if (ParameterInput == nullptr)
	{
		return false;
	}
	else if (ParameterInput->ClipboardFunctionInput == nullptr)
	{
		return false;
	}
	
	const FNiagaraTypeDefinition* InputTypeDef = InputNameToTypeDefMap.Find(ParameterName);
	if (InputTypeDef == nullptr)
	{
		Log("Failed to set parameter " + ParameterName + ": Could not find input with this name!", ENiagaraMessageSeverity::Error);
		return false;
	}
	else if (ParameterInput->TypeDefinition != *InputTypeDef)
	{
		Log("Failed to set parameter " + ParameterName + ": Input types did not match! /n Tried to set: " + ParameterInput->TypeDefinition.GetName() + " | Input type was: " + InputTypeDef->GetName(), ENiagaraMessageSeverity::Error);
		return false;
	}

	ParameterInput->ClipboardFunctionInput->bHasEditCondition = bInHasEditCondition;
	ParameterInput->ClipboardFunctionInput->bEditConditionValue = bInEditConditionValue;
	ParameterInput->ClipboardFunctionInput->InputName = FName(*ParameterName);
	FunctionInputs.Add(ParameterInput->ClipboardFunctionInput);
	StackMessages.Append(ParameterInput->StackMessages);
	return true;
}

void UNiagaraScriptConversionContext::Log(FString Message, ENiagaraMessageSeverity Severity, bool bIsVerbose /* = false*/)
{	
	StackMessages.Add(FGenericConverterMessage(Message, Severity, bIsVerbose));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	UNiagaraScriptConversionContextInput																	  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void UNiagaraScriptConversionContextInput::Init(
	  UNiagaraClipboardFunctionInput* InClipboardFunctionInput
	, const ENiagaraScriptInputType InInputType
	, const FNiagaraTypeDefinition& InTypeDefinition)
{
	ClipboardFunctionInput = InClipboardFunctionInput;
	InputType = InInputType;
	TypeDefinition = InTypeDefinition;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	Wrapper Structs																							  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
TArray<FRichCurveKey> FRichCurveKeyBP::KeysToBase(const TArray<FRichCurveKeyBP>& InKeyBPs)
{
	TArray<FRichCurveKey> Keys;
	Keys.AddUninitialized(InKeyBPs.Num());
	for (int i = 0; i < InKeyBPs.Num(); ++i)
	{
		Keys[i] = InKeyBPs[i];
	}
	return Keys;
}

FNiagaraEventScriptProperties FNiagaraEventHandlerAddAction::GetEventScriptProperties() const
{
	FNiagaraEventScriptProperties EventScriptProperties;
	EventScriptProperties.ExecutionMode = ExecutionMode;
	EventScriptProperties.SpawnNumber = SpawnNumber;
	EventScriptProperties.MaxEventsPerFrame = MaxEventsPerFrame;
	EventScriptProperties.SourceEmitterID = SourceEmitterID;
	EventScriptProperties.SourceEventName = SourceEventName;
	EventScriptProperties.bRandomSpawnNumber = bRandomSpawnNumber;
	EventScriptProperties.MinSpawnNumber = MinSpawnNumber;
	return EventScriptProperties;
}

