// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FExtender;
class FMenuBuilder;
struct FAssetData;
class UParticleSystem;

DECLARE_LOG_CATEGORY_EXTERN(LogFXConverter, Log, Verbose);

struct FNiagaraConverterMessageTopics
{
	static const FName VerboseConversionEventTopicName;
	static const FName ConversionEventTopicName;
};

class ICascadeToNiagaraConverterModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);
	static void AddMenuExtenderConvertEntry(FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets);
	static void ExecuteConvertCascadeSystemToNiagaraSystem(TArray<UParticleSystem*> CascadeSystems);
};
