// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Curves/RichCurve.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/PackageName.h"
#include "NiagaraEmitter.h"
#include "Particles/Location/ParticleModuleLocationBoneSocket.h"
#include "Particles/Orbit/ParticleModuleOrbit.h"
#include "Particles/Parameter/ParticleModuleParameterDynamic.h"
#include "Particles/ParticleEmitter.h"
#include "NiagaraStackGraphUtilitiesAdapterLibrary.generated.h"

class UDistributionFloatConstantCurve;
class UDistributionFloatParameterBase;
class UDistributionFloatUniform;
class UDistributionFloatUniformCurve;
class UDistributionVectorConstant;
class UDistributionVectorConstantCurve;
class UDistributionVectorParameterBase;
class UDistributionVectorUniform;
class UDistributionVectorUniformCurve;
class UNiagaraDataInterfaceCurve;
class UNiagaraDataInterfaceVector2DCurve;
class UNiagaraDataInterfaceVector4Curve;
class UNiagaraDataInterfaceVectorCurve;
class UNiagaraEmitterConversionContext;
class UNiagaraMeshRendererProperties;
class UNiagaraNodeFunctionCall;
class UNiagaraRibbonRendererProperties;
class UParticleModuleAcceleration;
class UParticleModuleAccelerationConstant;
class UParticleModuleAccelerationDrag;
class UParticleModuleAccelerationDragScaleOverLife;
class UParticleModuleAccelerationOverLifetime;
class UParticleModuleAttractorLine;
class UParticleModuleAttractorParticle;
class UParticleModuleAttractorPoint;
class UParticleModuleCameraOffset;
class UParticleModuleCollision;
class UParticleModuleColor;
class UParticleModuleColorOverLife;
class UParticleModuleColorScaleOverLife;
class UParticleModuleKillBox;
class UParticleModuleLifetime;
class UParticleModuleLight;
class UParticleModuleLocation;
class UParticleModuleLocationDirect;
class UParticleModuleLocationPrimitiveBase;
class UParticleModuleLocationPrimitiveCylinder;
class UParticleModuleLocationPrimitiveSphere;
class UParticleModuleLocationWorldOffset;
class UParticleModuleLocationSkelVertSurface;
class UParticleModuleMeshMaterial;
class UParticleModuleMeshRotation;
class UParticleModuleMeshRotationRate;
class UParticleModuleMeshRotationRateMultiplyLife;
class UParticleModuleOrientationAxisLock;
class UParticleModulePivotOffset;
class UParticleModuleRotation;
class UParticleModuleRotationOverLifetime;
class UParticleModuleRotationRate;
class UParticleModuleRotationRateMultiplyLife;
class UParticleModuleSize;
class UParticleModuleSizeMultiplyLife;
class UParticleModuleSizeScale;
class UParticleModuleSizeScaleBySpeed;
class UParticleModuleSpawn;
class UParticleModuleSubUV;
class UParticleModuleSubUVMovie;
class UParticleModuleTypeDataGpu;
class UParticleModuleVectorFieldLocal;
class UParticleModuleVectorFieldRotationRate;
class UParticleModuleVelocity;
class UParticleModuleVelocityCone;
class UParticleModuleVelocityInheritParent;
class UParticleModuleVelocityOverLifetime;
class USkeletalMesh;
class USubUVAnimation;
class UUserDefinedStruct;
class UVectorField;
enum CylinderHeightAxis : int;
enum EAttractorParticleSelectionMethod : int;
enum EDistributionVectorLockFlags : int;
enum EEmitterNormalsMode : int;
enum EMeshCameraFacingOptions : int;
enum EMeshCameraFacingUpAxis : int;
enum EMeshScreenAlignment : int;
enum EObjectTypeQuery : int;
enum EOpacitySourceMode : int;
enum EParticleCameraOffsetUpdateMethod : int;
enum EParticleCollisionComplete : int;
enum EParticleScreenAlignment : int;
enum EParticleSortMode : int;
enum EParticleSourceSelectionMethod : int;
enum ESubUVBoundingVertexCount : int;
enum ETrail2SourceMethod : int;
enum ETrailsRenderAxisOption : int;
enum ELocationSkelVertSurfaceSource : int;
enum class ENiagaraMessageSeverity : uint8;
enum class ENiagaraRendererSourceDataMode : uint8;
enum class ENiagaraSimTarget : uint8;
struct FInterpCurveFloat;
struct FInterpCurveTwoVectors;
struct FInterpCurveVector2D;
struct FInterpCurveVector;
struct FLightingChannels;
class UNiagaraScriptConversionContextInput;
class UNiagaraScriptConversionContext;
class UNiagaraSystem;
class UNiagaraEmitter;
class UNiagaraRendererProperties;
class UNiagaraLightRendererProperties;
class UNiagaraComponentRendererProperties;
class UParticleEmitter;
class UParticleLODLevel;
class UParticleModule;
class FNiagaraEmitterHandleViewModel;
class FNiagaraSystemViewModel;
class UNiagaraClipboardFunction;
class UNiagaraClipboardFunctionInput;
class UDistributionFloat;
class UNiagaraDataInterfaceSkeletalMesh;

struct FParticleBurst;
struct FNiagaraMeshMaterialOverride;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	Enums																									  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
UENUM()
enum class EScriptExecutionCategory : uint8
{
	EmitterSpawn = 0,
	EmitterUpdate,
	ParticleSpawn,
	ParticleUpdate,
	ParticleEvent UMETA(Hidden),

	NONE
};

UENUM()
enum class EDistributionType : uint8
{
	Const = 0,
	ConstCurve,
	Uniform,
	UniformCurve,
	Parameter,

	NONE
};

UENUM()
enum class EDistributionValueType : uint8
{
	Float = 0,
	Vector,

	NONE
};

UENUM()
enum class ECascadeRendererType : uint8
{ 
	Sprite = 0,
	Mesh,
	Ribbon,
	Beam,
	AnimTrail,

	NONE
};

UENUM()
enum class ENiagaraScriptInputType : uint8
{
	Int = 0,
	Float,
	Vec2,
	Vec3,
	Vec4,
	LinearColor,
	Matrix,
	Quaternion,
	Struct,
	Enum,
	DataInterface,
	Bool,
	Position,

	NONE
};

UENUM()
enum class ERibbonConversionMode : uint8
{
	Event = 0,
	DirectRead,

	NONE
};

UENUM()
enum class EStackEntryAddActionMode : uint8
{
	Module = 0,
	SetParameter,

	NONE
};

UENUM()
enum class ENiagaraEventHandlerAddMode : uint8
{
	AddEvent,
	AddEventAndEventGenerator,

	NONE
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	Wrapper Structs																							  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


/** BlueprintType wrapper around FEmitterDynamicParameter to allow managing in blueprint/python logic. */
USTRUCT(BlueprintInternalUseOnly)
struct FEmitterDynamicParameterBP
{
	GENERATED_USTRUCT_BODY()

	/** The parameter name - from the material DynamicParameter expression. READ-ONLY */
	UPROPERTY(BlueprintReadWrite, Category = EmitterDynamicParameter)
	FName ParamName;

	/** If true, use the EmitterTime to retrieve the value, otherwise use Particle RelativeTime. */
	UPROPERTY(BlueprintReadWrite, Category = EmitterDynamicParameter)
	bool bUseEmitterTime;

	/** If true, only set the value at spawn time of the particle, otherwise update each frame. */
	UPROPERTY(BlueprintReadWrite, Category = EmitterDynamicParameter)
	bool bSpawnTimeOnly;

	/** Where to get the parameter value from. */
	UPROPERTY(BlueprintReadWrite, Category = EmitterDynamicParameter)
	TEnumAsByte<EEmitterDynamicParameterValue> ValueMethod;

	/** If true, scale the velocity value selected in ValueMethod by the evaluated ParamValue. */
	UPROPERTY(BlueprintReadWrite, Category = EmitterDynamicParameter)
	bool bScaleVelocityByParamValue;

	/** The distribution for the parameter value. */
	UPROPERTY(EditAnywhere, Category = EmitterDynamicParameter)
	struct FRawDistributionFloat ParamValue;


	FEmitterDynamicParameterBP()
		: bUseEmitterTime(false)
		, bSpawnTimeOnly(false)
		, ValueMethod(0)
		, bScaleVelocityByParamValue(false)
	{
	}
	FEmitterDynamicParameterBP(FName InParamName, uint32 InUseEmitterTime, TEnumAsByte<EEmitterDynamicParameterValue> InValueMethod, UDistributionFloatConstant* InDistribution)
		: ParamName(InParamName)
		, bUseEmitterTime((bool)InUseEmitterTime)
		, bSpawnTimeOnly(false)
		, ValueMethod(InValueMethod)
		, bScaleVelocityByParamValue(false)
	{
		ParamValue.Distribution = InDistribution;
	}

	FEmitterDynamicParameterBP(const FEmitterDynamicParameter& DynamicParameter)
		: ParamName(DynamicParameter.ParamName)
		, bUseEmitterTime(DynamicParameter.bUseEmitterTime)
		, bSpawnTimeOnly(DynamicParameter.bSpawnTimeOnly)
		, ValueMethod(DynamicParameter.ValueMethod)
		, bScaleVelocityByParamValue(DynamicParameter.bScaleVelocityByParamValue)
	{
		ParamValue.Distribution = DynamicParameter.ParamValue.Distribution;
	}
};

/** BlueprintType wrapper around FOrbitOptions to allow managing in blueprint/python logic. */
USTRUCT(BlueprintInternalUseOnly)
struct FOrbitOptionsBP
{
	GENERATED_USTRUCT_BODY()

	/**
	 *	Whether to process the data during spawning.
	 */
	UPROPERTY(EditAnywhere, Category=OrbitOptions)
	bool bProcessDuringSpawn;

	/**
	 *	Whether to process the data during updating.
	 */
	UPROPERTY(EditAnywhere, Category=OrbitOptions)
	bool bProcessDuringUpdate;

	/**
	 *	Whether to use emitter time during data retrieval.
	 */
	UPROPERTY(EditAnywhere, Category=OrbitOptions)
	bool bUseEmitterTime;

	FOrbitOptionsBP()
		: bProcessDuringSpawn(true)
		, bProcessDuringUpdate(false)
		, bUseEmitterTime(false)
	{
	}

	FOrbitOptionsBP(const FOrbitOptions& OrbitOptions)
		: bProcessDuringSpawn(OrbitOptions.bProcessDuringSpawn)
		, bProcessDuringUpdate(OrbitOptions.bProcessDuringUpdate)
		, bUseEmitterTime(OrbitOptions.bUseEmitterTime)
	{
	}
};

/** BlueprintType wrapper around FParticleBurst to allow managing in blueprint/python logic. */
USTRUCT(BlueprintInternalUseOnly)
struct FParticleBurstBlueprint
{
	GENERATED_USTRUCT_BODY()

	FParticleBurstBlueprint()
	{
		Count = 0;
		CountLow = 0;
		Time = 0.0f;
	};

	FParticleBurstBlueprint(const FParticleBurst& InParticleBurst)
		: Count(InParticleBurst.Count)
		, CountLow(InParticleBurst.CountLow)
		, Time(InParticleBurst.Time)
	{};

	/** The number of particles to burst */
	UPROPERTY(BlueprintReadWrite, Category = ParticleBurst)
	int32 Count;

	/** If >= 0, use as a range [CountLow..Count] */
	UPROPERTY(BlueprintReadWrite, Category = ParticleBurst)
	int32 CountLow;

	/** The time at which to burst them (0..1: emitter lifetime) */
	UPROPERTY(BlueprintReadWrite, Category = ParticleBurst)
	float Time;
};

USTRUCT(BlueprintInternalUseOnly)
struct FRichCurveKeyBP : public FRichCurveKey
{
	GENERATED_BODY()

	FRichCurveKeyBP()
		:FRichCurveKey()
	{};

	FRichCurveKeyBP(const FRichCurveKey& Other) 
		:FRichCurveKey(Other) 
	{};

	static TArray<FRichCurveKey> KeysToBase(const TArray<FRichCurveKeyBP>& InKeyBPs);
};

USTRUCT(BlueprintInternalUseOnly)
struct FLocationBoneSocketInfoBP : public FLocationBoneSocketInfo
{
	GENERATED_BODY()

	FLocationBoneSocketInfoBP()
		:FLocationBoneSocketInfo()
	{};

	FLocationBoneSocketInfoBP(const FLocationBoneSocketInfo& Other)
		:FLocationBoneSocketInfo(Other)
	{};
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraAddEventGeneratorOptions
{
	GENERATED_BODY()

	FNiagaraAddEventGeneratorOptions() = default;

	/** The name to lookup the emitter which has the event generator. */
	UPROPERTY(BlueprintReadWrite, Category = "Event Generator Options")
	FName SourceEmitterName;

	/** AssetData pointing to the UNiagaraScript that generates the event. */
	UPROPERTY(BlueprintReadWrite, Category = "Event Generator Options")
	FAssetData EventGeneratorScriptAssetData;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraEventHandlerAddAction
{
	GENERATED_BODY()

	FNiagaraEventHandlerAddAction()
		: Mode(ENiagaraEventHandlerAddMode::AddEvent)
		, AddEventGeneratorOptions()
		, ExecutionMode(EScriptExecutionMode::EveryParticle)
		, SpawnNumber(0)
		, MaxEventsPerFrame(0)
		, SourceEmitterID()
		, SourceEventName()
		, bRandomSpawnNumber(false)
		, MinSpawnNumber(0)
	{}

	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	ENiagaraEventHandlerAddMode Mode;

	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	FNiagaraAddEventGeneratorOptions AddEventGeneratorOptions;

	// Begin FNiagaraEventScriptProperties
	/** Controls which particles have the event script run on them.*/
	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	EScriptExecutionMode ExecutionMode;

	/** Controls whether or not particles are spawned as a result of handling the event. Only valid for EScriptExecutionMode::SpawnedParticles. If Random Spawn Number is used, this will act as the maximum spawn range. */
	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	int32 SpawnNumber;

	/** Controls how many events are consumed by this event handler. If there are more events generated than this value, they will be ignored.*/
	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	int32 MaxEventsPerFrame;

	/** Id of the Emitter Handle that generated the event. If all zeroes, the event generator is assumed to be this emitter.*/
	UPROPERTY()
	FGuid SourceEmitterID;

	/** The name of the event generated. This will be "Collision" for collision events and the Event Name field on the DataSetWrite node in the module graph for others. */
	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	FName SourceEventName;

	/** Whether using a random spawn number. */
	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	bool bRandomSpawnNumber;

	/** The minimum spawn number when random spawn is used. Spawn Number is used as the maximum range. */
	UPROPERTY(BlueprintReadWrite, Category = "Event Handler Options")
	int32 MinSpawnNumber;
	// End FNiagaraEventScriptProperties

	FNiagaraEventScriptProperties GetEventScriptProperties() const;
};

USTRUCT(BlueprintType)
struct FStackEntryID
{
	GENERATED_BODY()

	FStackEntryID()
		: ScriptExecutionCategory(EScriptExecutionCategory::EmitterSpawn)
		, EventName()
	{};

	FStackEntryID(EScriptExecutionCategory TargetExecutionCategory)
		: ScriptExecutionCategory(TargetExecutionCategory)
		, EventName()
	{};

	FStackEntryID(const FNiagaraEventHandlerAddAction& TargetEventHandlerStackEntry)
		: ScriptExecutionCategory(EScriptExecutionCategory::ParticleEvent)
		, EventName(TargetEventHandlerStackEntry.SourceEventName)
	{};

	// The stack group to put a stack entry in.
	UPROPERTY()
	EScriptExecutionCategory ScriptExecutionCategory;

	// If the ScriptExecutionCategory is ParticleEvent, this FName is the key to find the desired event stack group.
	UPROPERTY()
	FName EventName;
};

USTRUCT()
struct FStackEntryAddAction
{
	GENERATED_BODY()

		FStackEntryAddAction() = default;

	FStackEntryAddAction(UNiagaraScriptConversionContext* InScriptConversionContext, const FStackEntryID& InStackEntryID, const FName& InModuleName)
		: Mode(EStackEntryAddActionMode::Module)
		, ScriptConversionContext(InScriptConversionContext)
		, StackEntryID(InStackEntryID)
		, ModuleName(InModuleName)
	{};

	FStackEntryAddAction(UNiagaraClipboardFunction* InParameterSetClipboardFunction, const FStackEntryID& InStackEntryID)
		: Mode(EStackEntryAddActionMode::SetParameter)
		, ClipboardFunction(InParameterSetClipboardFunction)
		, StackEntryID(InStackEntryID)
	{};

	// The mode of the AddStackEntryAction: when the action is applied to a UNiagaraEmitterConversionContext during Finalize(), the mode chooses what form of stack entry to add.
	UPROPERTY()
	EStackEntryAddActionMode Mode = EStackEntryAddActionMode::Module;

	// If Mode is Module, represents the pending module script to create a stack entry for. Otherwise this value is ignored.
	UPROPERTY()
	TObjectPtr<UNiagaraScriptConversionContext> ScriptConversionContext = nullptr;

	// If mode is SetParameter, represents the pending parameter to set directly and create a stack entry for. Otherwise this value is ignored.
	UPROPERTY()
	TObjectPtr<UNiagaraClipboardFunction> ClipboardFunction = nullptr;

	// Info to find the category of the stack to add the stack entry to.
	UPROPERTY()
	FStackEntryID StackEntryID;

	// If Mode is Module, acts as a key to lookup the ScriptConversionContext.
	UPROPERTY()
	FName ModuleName;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraScriptVersion
{
	GENERATED_BODY()

public:
	FNiagaraScriptVersion() = default;

	FNiagaraScriptVersion(const int32 InMajorVersion, const int32 InMinorVersion)
		: MajorVersion(InMajorVersion)
		, MinorVersion(InMinorVersion)
	{};

	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	int32 MajorVersion = 0;

	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	int32 MinorVersion = 0;
};

USTRUCT(BlueprintInternalUseOnly)
struct FCreateScriptContextArgs
{
	GENERATED_BODY()

public:
	FCreateScriptContextArgs() = default;

	FCreateScriptContextArgs(FAssetData InScriptAsset)
		: ScriptAsset(InScriptAsset)
		, ScriptVersion()
	{};

	FCreateScriptContextArgs(FAssetData InScriptAsset, FNiagaraScriptVersion InScriptVersion)
		: ScriptAsset(InScriptAsset)
		, bScriptVersionSet(true)
		, ScriptVersion(InScriptVersion)
	{};

	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	FAssetData ScriptAsset;

	UPROPERTY()
	bool bScriptVersionSet = false;

	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	FNiagaraScriptVersion ScriptVersion;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	Logging Framework																						  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT(BlueprintInternalUseOnly)
struct FGenericConverterMessage
{
	GENERATED_BODY()

public:
	FGenericConverterMessage() {};

	FGenericConverterMessage(FString InMessage, ENiagaraMessageSeverity InMessageSeverity, bool bInIsVerbose)
		: Message(InMessage)
		, MessageSeverity(InMessageSeverity)
		, bIsVerbose(bInIsVerbose)
	{};

	FString Message;
	ENiagaraMessageSeverity MessageSeverity;
	bool bIsVerbose;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	Wrapper Classes																							  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Wrapper for modifying a UNiagaraSystem by adding Emitters through UNiagaraEmitterConversionContexts.
 */
UCLASS(BlueprintInternalUseOnly)
class UNiagaraSystemConversionContext : public UObject
{
	GENERATED_BODY()

public:
	UNiagaraSystemConversionContext() {};

	/**
	 * Init the System Conversion Context.
	 * @param InSystem				The System to convert.
	 * @param InSystemViewModel		The FNiagaraSystemViewModel pointing at the InSystem.
	 */
	void Init(UNiagaraSystem* InSystem, const TSharedPtr<FNiagaraSystemViewModel>& InSystemViewModel)
	{
		System = InSystem;
		SystemViewModel = InSystemViewModel;
	}

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void Cleanup();

	/** Add an empty emitter to the system and return an emitter conversion context. */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	UNiagaraEmitterConversionContext* AddEmptyEmitter(FString NewEmitterNameString);

	/** Add an emitter based on the specified template to the system and return an emitter conversion context. */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	UNiagaraEmitterConversionContext* AddTemplateEmitter(FString NewEmitterNameString, FString Template);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetWarmupTime(float InWarmupTime);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetWarmupTickDelta(float InWarmupTickDelta);

	/**
	 * Apply all pending UNiagaraScriptConversionContexts and UNiagaraRendererProperties to the owned
	 * UNiagaraEmitterContexts by creating clipboard inputs and pasting them onto the emitter conversion context's
	 * Emitter.
	 * Afterwards compile the system so the changes take effect.
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void Finalize();

	UNiagaraEmitterConversionContext* FindEmitterConversionContextByName(const FName& EmitterName);

private:
	UPROPERTY()
	TObjectPtr<UNiagaraSystem> System;

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;

	UPROPERTY()
	TMap<FName, TObjectPtr<UNiagaraEmitterConversionContext>> EmitterNameToConversionContextMap;
};

/** 
 * Wrapper for modifying a UNiagaraEmitter by adding Scripts and Renderers through UNiagaraScriptConversionContexts and 
 * UNiagaraRendererProperties, respectively. 
 */
UCLASS(BlueprintInternalUseOnly)
class UNiagaraEmitterConversionContext : public UObject
{
	GENERATED_BODY()

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnPasteScript, int32, ScriptIdx);

public:
	UNiagaraEmitterConversionContext()
		: bEnabled(false)
		, PastedFunctionCallNode(nullptr)
	{};

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void Cleanup();

	/** 
	 * Find or add a script conversion context to this emitter conversion context. If a script conversion context
	 * is not found by name string then a new one is created and initialized from the NiagaraScriptAssetData.
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	UNiagaraScriptConversionContext* FindOrAddModuleScript(
		FString ScriptNameString
		, FCreateScriptContextArgs CreateScriptContextArgs
		, EScriptExecutionCategory ModuleScriptExecutionCategory
	);

	/**
	 * Find a module script conversion context or add a module script conversion context  to this emitter conversion context for an event category. If a script conversion context
	 * is not found by name string then a new one is created and initialized from the NiagaraScriptAssetData.
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	UNiagaraScriptConversionContext* FindOrAddModuleEventScript(
		FString ScriptNameString
		, FCreateScriptContextArgs CreateScriptContextArgs
		, FNiagaraEventHandlerAddAction EventScriptProps
	);

	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	UNiagaraScriptConversionContext* FindModuleScript(FString ScriptNameString);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void AddModuleScript(
		FString ScriptNameString
		, UNiagaraScriptConversionContext* ScriptConversionContext
		, EScriptExecutionCategory ModuleScriptExecutionCategory
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void AddModuleEventScript(
		FString ScriptNameString
		, UNiagaraScriptConversionContext* ScriptConversionContext
		, FNiagaraEventHandlerAddAction EventScriptProps
	);

	/**
	 * Add a set parameter module to the emitter handled by this emitter conversion context.
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetParameterDirectly(
		FString ParameterNameString
		, UNiagaraScriptConversionContextInput* ParameterInput
		, EScriptExecutionCategory SetParameterExecutionCategory
	);

	/** Add a renderer to this emitter conversion context through renderer properties. */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void AddRenderer(FString RendererNameString, UNiagaraRendererProperties* NewRendererProperties);

	/** Find an added renderer properties by name string. */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	UNiagaraRendererProperties* FindRenderer(FString RendererNameString);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	TArray<UNiagaraRendererProperties*> GetAllRenderers();

	/**
	 * Log a message to the niagara message log for the emitter.
	 * @param Message		The message string to display.
	 * @param Severity		The severity of the message.
	 * @param bIsVerbose	Whether the message is verbose and should be displayed conditionally.
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void Log(FString Message, ENiagaraMessageSeverity Severity, bool bIsVerbose = false);

	/** 
	 * Apply all pending UNiagaraScriptConversionContexts and UNiagaraRendererProperties to this
	 * UNiagaraEmitterContext by creating clipboard inputs and pasting them onto the emitter conversion context's
	 * Emitter.
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void Finalize();

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	UNiagaraEmitter* GetEmitter() {return Emitter.Emitter;}

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetSimTarget(ENiagaraSimTarget InTarget) { if (GetEmitter() && GetEmitter()->GetLatestEmitterData()) { GetEmitter()->GetLatestEmitterData()->SimTarget = InTarget; } }

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetLocalSpace(bool bLocalSpace) {if (GetEmitter() && GetEmitter()->GetLatestEmitterData()) { GetEmitter()->GetLatestEmitterData()->bLocalSpace = bLocalSpace; }}

	void InternalFinalizeEvents(UNiagaraSystemConversionContext* OwningSystemConversionContext);

	void InternalFinalizeStackEntryAddActions();

	/**
	 * Init the Emitter Conversion Context. 
	 * @param InEmitter						The Emitter to convert.
	 * @param InEmitterHandleViewModelGuid	A Guid key to the FNiagaraEmitterHandleViewModel pointing at the InEmitter.
	 */
	void Init(FVersionedNiagaraEmitter InEmitter, const TSharedPtr<FNiagaraEmitterHandleViewModel>& InEmitterHandleViewModel)
	{
		Emitter = InEmitter;
		EmitterHandleViewModel = InEmitterHandleViewModel;
		bEnabled = true;
	};

	FGuid GetEmitterHandleId() const;

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetEnabled(bool bInEnabled) {bEnabled = bInEnabled;};

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	bool GetEnabled() const {return bEnabled;};

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetDetailBitMask(int32 BitMask);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void AddEventHandler(FNiagaraEventHandlerAddAction EventScriptPropertiesBP);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetRendererBinding(UNiagaraRendererProperties* InRendererProperties, FName BindingName, FName VariableToBindName, ENiagaraRendererSourceDataMode SourceDataMode);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void RemoveModuleScriptsForAssets(TArray<FAssetData> ScriptsToRemove);

private:
	UPROPERTY()
	FVersionedNiagaraEmitter Emitter;

	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;

	TArray<FNiagaraEventHandlerAddAction> EventHandlerAddActions;

	UPROPERTY()
	TArray<FStackEntryAddAction> StackEntryAddActions;

	UPROPERTY()
	TMap<FString, TObjectPtr<UNiagaraRendererProperties>> RendererNameToStagedRendererPropertiesMap;

	TArray<FGenericConverterMessage> EmitterMessages;

	bool bEnabled;

	UPROPERTY()
	TObjectPtr<UNiagaraNodeFunctionCall> PastedFunctionCallNode;

	UFUNCTION()
	void SetPastedFunctionCallNode(UNiagaraNodeFunctionCall* InFunctionCallNode) {PastedFunctionCallNode = InFunctionCallNode;};

	UNiagaraScriptConversionContext* PrivateFindOrAddModuleScript(
		const FString& ScriptNameString
		, const FCreateScriptContextArgs& CreateScriptContextArgs
		, const FStackEntryID& StackEntryID
	);
};

/** Wrapper for programmatically adding scripts to a UNiagaraEmitter through a UNiagaraEmitterConversionContext. */
UCLASS(BlueprintInternalUseOnly)
class UNiagaraScriptConversionContext : public UObject
{
	GENERATED_BODY()

public:
	UNiagaraScriptConversionContext() 
		: TargetExecutionCategory(EScriptExecutionCategory::NONE)
		, FunctionInputs()
		, Script(nullptr)
		, ScriptVersionGuid()
		, StackMessages()
		, InputNameToTypeDefMap()
		, bModuleEnabled(true)
	{};

	/** Init the Niagara Script Conversion Context with the assetdata to a UNiagaraScript. */
	void Init(const FAssetData& InNiagaraScriptAssetData, TOptional<FNiagaraScriptVersion> InNiagaraScriptVersion = TOptional<FNiagaraScriptVersion>());

	/** 
	 * Set a parameter on the Script this Script Conversion Context holds. 
	 * @param ParameterName		The target parameter name.
	 * @param ParameterInput	Value to set on the parameter.
	 * @return Whether setting the parameter was successful. 
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	bool SetParameter(FString ParameterName, UNiagaraScriptConversionContextInput* ParameterInput, bool bInHasEditCondition = false, bool bInEditConditionValue = false);

	/**
	 * Log a message to the stack and the niagara message log for the module associated with this script.
	 * @param Message		The message string to display.
	 * @param Severity		The severity of the message.
	 * @param bIsVerbose	Whether the message is verbose and should be displayed conditionally.
	 */
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void Log(FString Message, ENiagaraMessageSeverity Severity, bool bIsVerbose = false);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	void SetModuleEnabled(bool bInModuleEnabled) {bModuleEnabled = bInModuleEnabled;};

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	bool GetModuleEnabled() const {return bModuleEnabled;};

	const TArray<const UNiagaraClipboardFunctionInput*>& GetClipboardFunctionInputs() { return FunctionInputs; };

	UNiagaraScript* GetScript() { return Script; };

	const FGuid& GetScriptVersionGuid() { return ScriptVersionGuid; };

	const TArray<FGenericConverterMessage>& GetStackMessages() const {return StackMessages;};

private:
	// Execution category to add this script to when it is finalized to a system or emitter.
	EScriptExecutionCategory TargetExecutionCategory;

	UPROPERTY()
	TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>> FunctionInputs;

	UPROPERTY()
	TObjectPtr<UNiagaraScript> Script;

	FGuid ScriptVersionGuid;

	TArray<FGenericConverterMessage> StackMessages;

	// Map of input variable names to their type defs for verifying inputs.
	TMap<FString, FNiagaraTypeDefinition> InputNameToTypeDefMap;

	bool bModuleEnabled;
};

/** Wrapper for setting the value on a parameter of a UNiagaraScript, applied through a UNiagaraScriptConversionContext. */
UCLASS(BlueprintInternalUseOnly)
class UNiagaraScriptConversionContextInput : public UObject
{
	GENERATED_BODY()

public:
	UNiagaraScriptConversionContextInput()
		: InputType(ENiagaraScriptInputType::NONE)
	{};

	UFUNCTION()
	void Init(UNiagaraClipboardFunctionInput* InClipboardFunctionInput, const ENiagaraScriptInputType InInputType, const FNiagaraTypeDefinition& InTypeDefinition);

	UPROPERTY()
	TObjectPtr<UNiagaraClipboardFunctionInput> ClipboardFunctionInput;

	UPROPERTY(BlueprintReadOnly, Category = StaticValue)
	ENiagaraScriptInputType InputType;

	UPROPERTY()
	FNiagaraTypeDefinition TypeDefinition;

	TArray<FGenericConverterMessage> StackMessages;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////	UFXConverterUtilitiesLibrary																			  /////
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/**
* A C++ and Blueprint accessible library for converting fx type assets (Cascade and Niagara)
*/
UCLASS(BlueprintType)
class UFXConverterUtilitiesLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Generic Utilities
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static FString GetLongPackagePath(const FString& InLongPackageName) { return FPackageName::GetLongPackagePath(InLongPackageName); }


	// Cascade Emitter and ParticleLodLevel Getters
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static TArray<UParticleEmitter*> GetCascadeSystemEmitters(const UParticleSystem* System);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static float GetWarmupTime(const UParticleSystem* System);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static UParticleLODLevel* GetCascadeEmitterLodLevel(UParticleEmitter* Emitter, const int32 Idx);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static bool GetLodLevelIsEnabled(UParticleLODLevel* LodLevel);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static TArray<UParticleModule*> GetLodLevelModules(UParticleLODLevel* LodLevel);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static UParticleModuleSpawn* GetLodLevelSpawnModule(UParticleLODLevel* LodLevel);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static UParticleModuleRequired* GetLodLevelRequiredModule(UParticleLODLevel* LodLevel);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static UParticleModuleTypeDataBase* GetLodLevelTypeDataModule(UParticleLODLevel* LodLevel);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static bool GetParticleModuleIsEnabled(UParticleModule* Module);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static FName GetCascadeEmitterName(UParticleEmitter* Emitter);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static int32 GetCascadeEmitterRenderMode(UParticleEmitter* Emitter);


	// Niagara Script and Script Input Helpers
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContext* CreateScriptContext(const FCreateScriptContextArgs& Args);


	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static FAssetData CreateAssetData(FString InPath);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputLinkedEnum(const FString& ParameterNameString, const FString& UserDefinedEnumAssetPath);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputLinkedParameter(FString ParameterNameString, ENiagaraScriptInputType InputType);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputLinkedParameterEx(FString ParameterNameString, UNiagaraScriptConversionContextInput* InBaseInput);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputFloat(float Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputLinearColor(FLinearColor Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputMatrix(FMatrix44f Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputQuat(FQuat4f Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputVec2(FVector2D Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputVec4(FVector4f Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputVector(FVector Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputStruct(UUserDefinedStruct* Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputEnum(const FString& UserDefinedEnumAssetPath, const FString& UserDefinedEnumValueNameString);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputInt(int32 Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputDynamic(UNiagaraScriptConversionContext* Value, ENiagaraScriptInputType InputType);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputDI(UNiagaraDataInterface* Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraScriptConversionContextInput* CreateScriptInputBool(bool Value);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraRibbonRendererProperties* CreateRibbonRendererProperties();

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraMeshRendererProperties* CreateMeshRendererProperties();

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraLightRendererProperties* CreateLightRendererProperties();

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraComponentRendererProperties* CreateComponentRendererProperties();


	// Niagara DI Helpers
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraDataInterfaceSkeletalMesh* CreateSkeletalMeshDataInterface();

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraDataInterfaceCurve* CreateFloatCurveDI(TArray<FRichCurveKeyBP> Keys);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraDataInterfaceVector2DCurve* CreateVec2CurveDI(TArray<FRichCurveKeyBP> X_Keys, TArray<FRichCurveKeyBP> Y_Keys);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraDataInterfaceVectorCurve* CreateVec3CurveDI(
		TArray<FRichCurveKeyBP> X_Keys,
		TArray<FRichCurveKeyBP> Y_Keys,
		TArray<FRichCurveKeyBP> Z_Keys
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UNiagaraDataInterfaceVector4Curve* CreateVec4CurveDI(
	TArray<FRichCurveKeyBP> X_Keys,
	TArray<FRichCurveKeyBP> Y_Keys,
	TArray<FRichCurveKeyBP> Z_Keys,
	TArray<FRichCurveKeyBP> W_Keys
	);


	// Niagara System and Emitter Helpers
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static UNiagaraSystemConversionContext* CreateSystemConversionContext(UNiagaraSystem* InSystem);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleTypeDataGpuProps(UParticleModuleTypeDataGpu* ParticleModule);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleTypeDataBeam2Props(UParticleModuleTypeDataBeam2* ParticleModule);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleTypeDataMeshProps(
			UParticleModuleTypeDataMesh* ParticleModule
			, UStaticMesh*& OutMesh
			, float& OutLODSizeScale
			, bool& bOutUseStaticMeshLODs
			, bool& bOutCastShadows
			, bool& bOutDoCollisions
			, TEnumAsByte<EMeshScreenAlignment>& OutMeshAlignment
			, bool& bOutOverrideMaterial
			, bool& bOutOverrideDefaultMotionBlurSettings
			, bool& bOutEnableMotionBlur
			, UDistribution*& OutRollPitchYawRange
			, TEnumAsByte<EParticleAxisLock>& OutAxisLockOption
			, bool& bOutCameraFacing
			, TEnumAsByte<EMeshCameraFacingUpAxis>& OutCameraFacingUpAxisOption_DEPRECATED
			, TEnumAsByte<EMeshCameraFacingOptions>& OutCameraFacingOption
			, bool& bOutApplyParticleRotationAsSpin
			, bool& bOutFacingCameraDirectionRatherThanPosition
			, bool& bOutCollisionsConsiderParticleSize
		);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static UClass* GetParticleModuleTypeDataRibbonClass();

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleTypeDataRibbonProps(
		UParticleModuleTypeDataRibbon* ParticleModule
		, int32& OutMaxTessellationBetweenParticles
		, int32& OutSheetsPerTrail
		, int32& OutMaxTrailCount
		, int32& OutMaxParticleInTrailCount
		, bool& bOutDeadTrailsOnDeactivate
		, bool& bOutClipSourceSegment
		, bool& bOutEnablePreviousTangentRecalculation
		, bool& bOutTangentRecalculationEveryFrame
		, bool& bOutSpawnInitialParticle
		, TEnumAsByte<ETrailsRenderAxisOption>& OutRenderAxis
		, float& OutTangentSpawningScalar
		, bool& bOutRenderGeometry
		, bool& bOutRenderSpawnPoints
		, bool& bOutRenderTangents
		, bool& bOutRenderTessellation
		, float& OutTilingDistance
		, float& OutDistanceTessellationStepSize
		, bool& bOutEnableTangentDiffInterpScale
		, float& OutTangentTessellationScalar
	);
	
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleSpawnProps(
		  UParticleModuleSpawn* ParticleModuleSpawn
		, UDistribution*& OutRate
		, UDistribution*& OutRateScale
		, TEnumAsByte<EParticleBurstMethod>& OutBurstMethod
		, TArray<FParticleBurstBlueprint>& OutBurstList
		, UDistribution*& OutBurstScale
		, bool& bOutApplyGlobalSpawnRateScale
		, bool& bOutProcessSpawnRate
		, bool& bOutProcessSpawnBurst
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleSpawnPerUnitProps(
		UParticleModuleSpawnPerUnit* ParticleModule
		, float& OutUnitScalar
		, float& OutMovementTolerance
		, UDistribution*& OutSpawnPerUnit
		, float& OutMaxFrameDistance
		, bool& bOutIgnoreSpawnRateWhenMoving
		, bool& bOutIgnoreMovementAlongX
		, bool& bOutIgnoreMovementAlongY
		, bool& bOutIgnoreMovementAlongZ
		, bool& bOutProcessSpawnRate
		, bool& bOutProcessBurstList
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleRequiredPerRendererProps(
		  UParticleModuleRequired* ParticleModuleRequired
		, UMaterialInterface*& OutMaterialInterface
		, TEnumAsByte<EParticleScreenAlignment>& OutScreenAlignment
		, int32& OutSubImages_Horizontal
		, int32& OutSubImages_Vertical
		, TEnumAsByte<EParticleSortMode>& OutSortMode
		, TEnumAsByte<EParticleSubUVInterpMethod>& OutInterpolationMethod
		, uint8& bOutRemoveHMDRoll
		, float& OutMinFacingCameraBlendDistance
		, float& OutMaxFacingCameraBlendDistance
		, UTexture2D*& OutCutoutTexture
		, TEnumAsByte<ESubUVBoundingVertexCount>& OutBoundingMode
		, TEnumAsByte<EOpacitySourceMode>& OutOpacitySourceMode
		, TEnumAsByte< EEmitterNormalsMode>& OutEmitterNormalsMode
		, float& OutAlphaThreshold
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleRequiredPerModuleProps(
		UParticleModuleRequired* ParticleModuleRequired
		, bool& bOutOrbitModuleAffectsVelocityAlignment
		, float& OutRandomImageTime
		, int32& OutRandomImageChanges
		, bool& bOutOverrideSystemMacroUV
		, FVector& OutMacroUVPosition
		, float& OutMacroUVRadius
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleRequiredPerEmitterProps(
		UParticleModuleRequired* ParticleModuleRequired
		, FVector& OutEmitterOrigin
		, FRotator& OutEmitterRotation
		, bool& bOutUseLocalSpace
		, bool& bOutKillOnDeactivate
		, bool& bOutKillOnCompleted
		, bool& bOutUseLegacyEmitterTime
		, bool& bOutEmitterDurationUseRange
		, float& OutEmitterDuration
		, float& OutEmitterDurationLow
		, bool& bOUtEmitterDelayUseRange
		, bool& bOutDelayFirstLoopOnly
		, float& OutEmitterDelay
		, float& OutEmitterDelayLow
		, bool& bOutDurationRecalcEachLoop
		, int32& OutEmitterLoops
	);
	
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleColorProps(UParticleModuleColor* ParticleModule, UDistribution*& OutStartColor, UDistribution*& OutStartAlpha, bool& bOutClampAlpha);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleColorOverLifeProps(UParticleModuleColorOverLife* ParticleModule, UDistribution*& OutColorOverLife, UDistribution*& OutAlphaOverLife, bool& bOutClampAlpha);
	
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleLifetimeProps(UParticleModuleLifetime* ParticleModule, UDistribution*& OutLifetime);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleSizeProps(UParticleModuleSize* ParticleModule, UDistribution*& OutStartSize);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleVelocityProps(UParticleModuleVelocity* ParticleModule, UDistribution*& OutStartVelocity, UDistribution*& OutStartVelocityRadial, bool& bOutInWorldSpace, bool& bOutApplyOwnerScale);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleVelocityConeProps(UParticleModuleVelocityCone* ParticleModule, UDistribution*& OutAngleDistribution, UDistribution*& OutVelocityDistribution, FVector& OutDirection, bool& bOutWorldSpace, bool& bOutApplyOwnerScale);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleVelocityOverLifetimeProps(
		UParticleModuleVelocityOverLifetime* ParticleModule
		, UDistribution*& OutVelOverLife
		, bool& bOutAbsolute
		, bool& bOutInWorldSpace
		, bool& bOutApplyOwnerScale
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleConstantAccelerationProps(UParticleModuleAccelerationConstant* ParticleModule, FVector& OutConstAcceleration);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleLocationPrimitiveSphereProps(
		UParticleModuleLocationPrimitiveSphere* ParticleModule
		, UDistribution*& OutStartRadius
		, bool& bOutPositiveX
		, bool& bOutPositiveY
		, bool& bOutPositiveZ
		, bool& bOutNegativeX
		, bool& bOutNegativeY
		, bool& bOutNegativeZ
		, bool& bOutSurfaceOnly
		, bool& bOutVelocity
		, UDistribution*& OutVelocityScale
		, UDistribution*& OutStartLocation
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleLocationPrimitiveCylinderProps(
		UParticleModuleLocationPrimitiveCylinder* ParticleModule
		, bool& bOutRadialVelocity
		, UDistribution*& OutStartRadius
		, UDistribution*& OutStartHeight
		, TEnumAsByte<CylinderHeightAxis>& OutHeightAxis
		, bool& bOutPositiveX
		, bool& bOutPositiveY
		, bool& bOutPositiveZ
		, bool& bOutNegativeX
		, bool& bOutNegativeY
		, bool& bOutNegativeZ
		, bool& bOutSurfaceOnly
		, bool& bOutVelocity
		, UDistribution*& OutVelocityScale
		, UDistribution*& OutStartLocation
	);
	
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleOrientationAxisLockProps(UParticleModuleOrientationAxisLock* ParticleModule, TEnumAsByte<EParticleAxisLock>& OutLockAxisFlags);
	
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleVelocityInheritParentProps(
		UParticleModuleVelocityInheritParent* ParticleModule
		, UDistribution*& OutScale
		, bool& bOutInWorldSpace
		, bool& bOutApplyOwnerScale
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleMeshRotationProps(UParticleModuleMeshRotation* ParticleModule, UDistribution*& OutStartRotation, bool& bOutInheritParentRotation);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleCollisionProps(
		UParticleModuleCollision* ParticleModule
		, UDistribution*& OutDampingFactor
		, UDistribution*& OutDampingFactorRotation
		, UDistribution*& OutMaxCollisions
		, TEnumAsByte<EParticleCollisionComplete>& OutCollisionCompleteOption
		, TArray<TEnumAsByte<EObjectTypeQuery>>& OutCollisionTypes
		, bool& bOutApplyPhysics
		, bool& bOutIgnoreTriggerVolumes
		, UDistribution*& OutParticleMass
		, float& OutDirScalar
		, bool& bOutPawnsDoNotDecrementCount
		, bool& bOutOnlyVerticalNormalsDecrementCount
		, float& OutVerticalFudgeFactor
		, UDistribution*& OutDelayAmount
		, bool& bOutDropDetail
		, bool& bOutCollideOnlyIfVisible
		, bool& bOutIgnoreSourceActor
		, float& OutMaxCollisionDistance
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleSizeScaleProps(UParticleModuleSizeScale* ParticleModule, UDistribution*& OutSizeScale);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleSizeScaleBySpeedProps(UParticleModuleSizeScaleBySpeed* ParticleModule, FVector2D& OutSpeedScale, FVector2D& OutMaxScale);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleVectorFieldLocalProps(
		  UParticleModuleVectorFieldLocal* ParticleModule
		, UVectorField* OutVectorField
		, FVector& OutRelativeTranslation
		, FRotator& OutRelativeRotation
		, FVector& OutRelativeScale3D
		, float& OutIntensity
		, float& OutTightness
		, bool& bOutIgnoreComponentTransform
		, bool& bOutTileX
		, bool& bOutTileY
		, bool& bOutTileZ
		, bool& bOutUseFixDT
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetParticleModuleVectorFieldRotationRateProps(UParticleModuleVectorFieldRotationRate* ParticleModule, FVector& OutRotationRate);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleOrbitProps(
		UParticleModuleOrbit* ParticleModule
		, TEnumAsByte<EOrbitChainMode>& OutChainMode
		, UDistribution*& OutOffsetAmount
		, FOrbitOptionsBP& OutOffsetOptions
		, UDistribution*& OutRotationAmount
		, FOrbitOptionsBP& OutRotationOptions
		, UDistribution*& OutRotationRateAmount
		, FOrbitOptionsBP& OutRotationRateOptions
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleSizeMultiplyLifeProps(
		UParticleModuleSizeMultiplyLife* ParticleModule
		, UDistribution*& OutLifeMultiplier
		, bool& OutMultiplyX
		, bool& OutMultiplyY
		, bool& OutMultiplyZ
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleColorScaleOverLifeProps(
		UParticleModuleColorScaleOverLife* ParticleModule
		, UDistribution*& OutColorScaleOverLife
		, UDistribution*& OutAlphaScaleOverLife
		, bool& bOutEmitterTime
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleRotationProps(UParticleModuleRotation* ParticleModule, UDistribution*& OutStartRotation);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleRotationRateProps(UParticleModuleRotationRate* ParticleModule, UDistribution*& OutStartRotationRate);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleRotationRateMultiplyLifeProps(UParticleModuleRotationRateMultiplyLife* ParticleModule, UDistribution*& OutLifeMultiplier);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleMeshRotationRateProps(UParticleModuleMeshRotationRate* ParticleModule, UDistribution*& OutStartRotationRate);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleRotationOverLifetimeProps(UParticleModuleRotationOverLifetime* ParticleModule, UDistribution*& OutRotationOverLife, bool& bOutScale);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleMeshRotationRateMultiplyLifeProps(UParticleModuleMeshRotationRateMultiplyLife* ParticleModule, UDistribution*& OutLifeMultiplier);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleMeshRotationRateOverLifeProps(UParticleModuleMeshRotationRateOverLife* ParticleModule, UDistribution*& OutRotRate, bool& bOutScaleRotRate);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModulePivotOffsetProps(UParticleModulePivotOffset* ParticleModule, FVector2D& OutPivotOffset);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleSubUVProps(
		UParticleModuleSubUV* ParticleModule
		, USubUVAnimation*& OutAnimation
		, UDistribution*& OutSubImageIndex
		, bool& bOutUseRealTime
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleCameraOffsetProps(
		UParticleModuleCameraOffset* ParticleModule
		, UDistribution*& OutCameraOffset
		, bool& bOutSpawnTimeOnly
		, TEnumAsByte<EParticleCameraOffsetUpdateMethod>& OutUpdateMethod
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleSubUVMovieProps(
		UParticleModuleSubUVMovie* ParticleModule
		, bool& bOutUseEmitterTime
		, UDistribution*& OutFrameRate
		, int32& OutStartingFrame
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleParameterDynamicProps(UParticleModuleParameterDynamic* ParticleModule, TArray<FEmitterDynamicParameterBP>& OutDynamicParams, bool& bOutUsesVelocity);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")	
	static void GetParticleModuleAccelerationDragProps(UParticleModuleAccelerationDrag* ParticleModule, UDistribution*& OutDragCoefficientRaw);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleAccelerationDragScaleOverLifeProps(UParticleModuleAccelerationDragScaleOverLife* ParticleModule, UDistribution*& OutDragScaleRaw);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleAccelerationProps(UParticleModuleAcceleration* ParticleModule, UDistribution*& OutAcceleration, bool& bOutApplyOwnerScale);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleAccelerationOverLifetimeProps(UParticleModuleAccelerationOverLifetime* ParticleModule, UDistribution*& OutAccelOverLife);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleTrailSourceProps(
		UParticleModuleTrailSource* ParticleModule
		, TEnumAsByte<ETrail2SourceMethod>& OutSourceMethod
		, FName& OutSourceName
		, UDistribution*& OutSourceStrength
		, bool& bOutLockSourceStrength
		, int32& OutSourceOffsetCount
		, TArray<FVector>& OutSourceOffsetDefaults
		, TEnumAsByte<EParticleSourceSelectionMethod>& OutSelectionMethod
		, bool& bOutInheritRotation
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleAttractorParticleProps(
		UParticleModuleAttractorParticle* ParticleModule
		, FName& OutEmitterName, UDistribution*& OutRange
		, bool& bOutStrengthByDistance
		, UDistribution*& OutStrength
		, bool& bOutAffectBaseVelocity
		, TEnumAsByte<EAttractorParticleSelectionMethod>& OutSelectionMethod
		, bool& bOutRenewSource
		, bool& bOutInheritSourceVelocity
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleAttractorPointProps(
		UParticleModuleAttractorPoint* ParticleModule
		, UDistribution*& OutPosition
		, UDistribution*& OutRange
		, UDistribution*& OutStrength
		, bool& boutStrengthByDistance
		, bool& bOutAffectsBaseVelocity
		, bool& bOutOverrideVelocity
		, bool& bOutUseWorldSpacePosition
		, bool& bOutPositiveX
		, bool& bOutPositiveY
		, bool& bOutPositiveZ
		, bool& bOutNegativeX
		, bool& bOutNegativeY
		, bool& bOutNegativeZ
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleAttractorLineProps(
		UParticleModuleAttractorLine* ParticleModule
		, FVector& OutStartPoint
		, FVector& OutEndPoint
		, UDistribution*& OutRange
		, UDistribution*& OutStrength
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleLocationDirectProps(
		UParticleModuleLocationDirect* ParticleModule
		, UDistribution*& OutLocation
		, UDistribution*& OutLocationOffset
		, UDistribution*& OutScaleFactor
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleLocationProps(
		UParticleModuleLocation* ParticleModule
		, UDistribution*& OutStartLocation
		, float& OutDistributeOverNPoints
		, float& OutDistributeThreshold
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleLocationBoneSocketProps(
		UParticleModuleLocationBoneSocket* ParticleModule
		, TEnumAsByte<ELocationBoneSocketSource>& OutSourceType
		, FVector& OutUniversalOffset
		, TArray<FLocationBoneSocketInfoBP>& OutSourceLocations
		, TEnumAsByte<ELocationBoneSocketSelectionMethod>& OutSelectionMethod
		, bool& bOutUpdatePositionEachFrame
		, bool& bOutOrientMeshEmitters
		, bool& bOutInheritBoneVelocity
		, float& OutInheritVelocityScale
		, FName& OutSkelMeshActorParamName
		, int32& OutNumPreSelectedIndices
		, USkeletalMesh*& OutEditorSkelMesh
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleLocationWorldOffsetProps(
		UParticleModuleLocationWorldOffset* ParticleModule
		, UDistribution*& OutStartLocation
		, float& OutDistributeOverNPoints
		, float& OutDistributeThreshold
	);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleLocationSkelVertSurfaceProps(
		UParticleModuleLocationSkelVertSurface* ParticleModule
		, TEnumAsByte<ELocationSkelVertSurfaceSource>& OutSourceType
		, FVector& OutUniversalOffset
		, bool& bOutUpdatePositionEachFrame
		, bool& bOutOrientMeshEmitters
		, bool& bOutInheritBoneVelocity
		, float& OutInheritVelocityScale
		, FName& OutSkelMeshActorParamName
		, USkeletalMesh*& OutEditorSkelMesh
		, TArray<FName>& OutValidAssociatedBones
		, bool& bOutEnforceNormalCheck
		, FVector& OutNormalToCompare
		, float& OutNormalCheckToleranceDegrees
		, float& OutNormalCheckTolerance
		, TArray<int32>& OutValidMaterialIndices
		, bool& bOutInheritVertexColor
		, bool& bOutInheritUV
		, int32& OutInheritUVChannel
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleKillBoxProps(
		UParticleModuleKillBox* ParticleModule
		, UDistribution*& OutLowerLeftCorner
		, UDistribution*& OutUpperRightCorner
		, bool& bOutWorldSpaceCoords
		, bool& bOutKillInside
		, bool& bOutAxisAlignedAndFixedSize
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleLightProps(
		UParticleModuleLight* ParticleModule
		, bool& bOutUseInverseSquaredFalloff
		, bool& bOutAffectsTranslucency
		, bool& bOutPreviewLightRadius
		, float& OutSpawnFraction
		, UDistribution*& OutColorScaleOverLife
		, UDistribution*& OutBrightnessOverLife
		, UDistribution*& OutRadiusScale
		, UDistribution*& OutLightExponent
		, FLightingChannels& OutLightingChannels
		, float& OutVolumetricScatteringIntensity
		, bool& bOutHighQualityLights
		, bool& bOutShadowCastingLights
	);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleMeshMaterialProps(UParticleModuleMeshMaterial* ParticleModule, TArray<UMaterialInterface*>& OutMeshMaterials);

	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleBeamSourceProps(UParticleModuleBeamSource* ParticleModule, UDistribution*& OutSource, bool& bOutSourceAbsolute);
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleBeamTargetProps(UParticleModuleBeamTarget* ParticleModule, UDistribution*& OutTarget, bool& bOutTargetAbsolute);
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void GetParticleModuleBeamNoiseProps(UParticleModuleBeamNoise* ParticleModule);
	
	UFUNCTION(BlueprintCallable, Category = "FXConverterUtilities")
	static void SetMeshRendererMaterialOverridesFromCascade(UNiagaraMeshRendererProperties* MeshRendererProps, TArray<UMaterialInterface*> MeshMaterials);
	
	// Cascade Distribution Getters
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetDistributionMinMaxValues(
		UDistribution* Distribution,
		bool& bOutSuccess,
		FVector& OutMinValue,
		FVector& OutMaxValue
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static TArray<TEnumAsByte<EDistributionVectorLockFlags>> GetDistributionLockedAxes(UDistribution* Distribution);
	
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetDistributionType(
		UDistribution* Distribution
		, EDistributionType& OutDistributionType
		, EDistributionValueType& OutCascadeDistributionValueType
	);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetFloatDistributionConstValues(UDistributionFloatConstant* Distribution, float& OutConstFloat);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetVectorDistributionConstValues(UDistributionVectorConstant* Distribution, FVector& OutConstVector);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetFloatDistributionUniformValues(UDistributionFloatUniform* Distribution, float& OutMin, float& OutMax);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetVectorDistributionUniformValues(UDistributionVectorUniform* Distribution, FVector& OutMin, FVector& OutMax);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetFloatDistributionConstCurveValues(UDistributionFloatConstantCurve* Distribution, FInterpCurveFloat& OutInterpCurveFloat);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetVectorDistributionConstCurveValues(UDistributionVectorConstantCurve* Distribution, FInterpCurveVector& OutInterpCurveVector);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetFloatDistributionUniformCurveValues(UDistributionFloatUniformCurve* Distribution, FInterpCurveVector2D& OutInterpCurveVector2D);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetVectorDistributionUniformCurveValues(UDistributionVectorUniformCurve* Distribution, FInterpCurveTwoVectors& OutInterpCurveTwoVectors);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetFloatDistributionParameterValues(UDistributionFloatParameterBase* Distribution, FName& OutParameterName, float& OutMinInput, float& OutMaxInput, float& OutMinOutput, float& OutMaxOutput);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static void GetVectorDistributionParameterValues(UDistributionVectorParameterBase* Distribution, FName& OutParameterName, FVector& OutMinInput, FVector& OutMaxInput, FVector& OutMinOutput, FVector& OutMaxOutput);


	// Cascade curve helpers
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static TArray<FRichCurveKeyBP> KeysFromInterpCurveFloat(FInterpCurveFloat Curve);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static TArray<FRichCurveKeyBP> KeysFromInterpCurveVector(FInterpCurveVector Curve, int32 ComponentIdx);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static TArray<FRichCurveKeyBP> KeysFromInterpCurveVector2D(FInterpCurveVector2D Curve, int32 ComponentIdx);

	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "FXConverterUtilities")
	static TArray<FRichCurveKeyBP> KeysFromInterpCurveTwoVectors(FInterpCurveTwoVectors Curve, int32 ComponentIdx);


	// Code only utilities
	static FName GetNiagaraScriptInputTypeName(ENiagaraScriptInputType InputType);

private:
	// Internal cascade module getters
	static void GetParticleModuleLocationPrimitiveBaseProps(
		UParticleModuleLocationPrimitiveBase* ParticleModule
		, bool& bOutPositiveX
		, bool& bOutPositiveY
		, bool& bOutPositiveZ
		, bool& bOutNegativeX
		, bool& bOutNegativeY
		, bool& bOutNegativeZ
		, bool& bOutSurfaceOnly
		, bool& bOutVelocity
		, UDistribution*& OutVelocityScale
		, UDistribution*& OutStartLocation
	);
};

/**
* Wrapper class for passing results back from the ConvertCascadeToNiagara python script.
*/
UCLASS(BlueprintType)
class UConvertCascadeToNiagaraResults : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION()
		void Init() { bCancelledByUser = false; bCancelledByPythonError = true; };

public:
	// Whether the converter process was cancelled by the user.
	UPROPERTY(BlueprintReadWrite, Category = "Results")
		bool bCancelledByUser;

	// Whether the converter process was cancelled due to an unrecoverable error in the python script process.
	UPROPERTY(BlueprintReadWrite, Category = "Results")
		bool bCancelledByPythonError;
};


